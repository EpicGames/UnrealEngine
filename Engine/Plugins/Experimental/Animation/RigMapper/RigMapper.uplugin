{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Rig Mapper",
	"Description": "A set of animation remapping features",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Plugins": [
		{
			"Name": "ControlRig",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "RigMapperEditor",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "RigMapper",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "RigMapperDeveloper",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault"
		}
	]
}
