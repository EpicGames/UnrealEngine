// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNode_RigMapper.h"

#include "Animation/AnimInstance.h"
#include "Components/SkeletalMeshComponent.h"
#include "RigMapperDefinition.h"
#include "Animation/AnimCurveUtils.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/AnimInstanceProxy.h"
#include "RigMapperLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_RigMapper)

DEFINE_STAT(STAT_RigMapperEvaluateNode);
DEFINE_STAT(STAT_RigMapperEvaluateRigs);
DEFINE_STAT(STAT_RigMapperInitialize);
DEFINE_STAT(STAT_RigMapperSetCurves);

FAnimNode_RigMapper::FAnimNode_RigMapper()
{
}

FAnimNode_RigMapper::~FAnimNode_RigMapper()
{
	
}

void FAnimNode_RigMapper::OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy,
                                                   const UAnimInstance* InAnimInstance)
{
	FAnimNode_Base::OnInitializeAnimInstance(InProxy, InAnimInstance);
}

void FAnimNode_RigMapper::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_Base::Initialize_AnyThread(Context);

	//GetEvaluateGraphExposedInputs().Execute(Context);
	
	SourcePose.Initialize(Context);
}

void FAnimNode_RigMapper::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);
	
	DebugLine += FString::Printf(TEXT("(Alpha: %f, Definitions: "), Alpha);
	for (int32 DefinitionIndex = 0; DefinitionIndex < Definitions.Num(); DefinitionIndex++)
	{
		DebugLine += FString::Printf(TEXT("%d: %s"), DefinitionIndex, Definitions[DefinitionIndex] ? *Definitions[DefinitionIndex]->GetName() : TEXT("NULL"));
	}
	DebugLine += ")";
	
	DebugData.AddDebugItem(DebugLine);
}

void FAnimNode_RigMapper::PreUpdate(const UAnimInstance* InAnimInstance)
{
	FAnimNode_Base::PreUpdate(InAnimInstance);
	
	// Need reinit if the definitions in use have changed.
	// todo: check against definition property changes? 
	bool bReInit = false;

	// If definitions were loaded from the SKM asset user data, they take priority and are the ones we will check against
	TArray<TObjectPtr<URigMapperDefinition>>* DefinitionsToCheck = &Definitions;
	if (LoadedUserData)
	{
		DefinitionsToCheck = &LoadedUserData->Definitions;
	}

	check(DefinitionsToCheck);

	// Otherwise, reinit if the definitions are not the same
	if (DefinitionsToCheck->Num() != LoadedDefinitions.Num())
	{
		bReInit = true;
	}
	else
	{
		for (int32 DefinitionIndex = 0; DefinitionIndex < DefinitionsToCheck->Num(); DefinitionIndex++)
		{
			if ((*DefinitionsToCheck)[DefinitionIndex] != LoadedDefinitions[DefinitionIndex] || !(*DefinitionsToCheck)[DefinitionIndex]->WasDefinitionValidated()) // check if individual definition has been changed or editted
			{
				bReInit = true;
				break;
			}
		}	
	}
	if (bReInit)
	{
		USkeletalMesh* TargetMesh = InAnimInstance && InAnimInstance->GetSkelMeshComponent() ? InAnimInstance->GetSkelMeshComponent()->GetSkeletalMeshAsset() : nullptr;
		InitializeRigMapping(TargetMesh);		
	}
	// todo: Allow override of user data (from BP etc...)
}

void FAnimNode_RigMapper::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Update_AnyThread)

	GetEvaluateGraphExposedInputs().Execute(Context);
	SourcePose.Update(Context);

	// todo: update cache if needed (config or skeleton curves changed)
}

void FAnimNode_RigMapper::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)

	FAnimNode_Base::CacheBones_AnyThread(Context);

	SourcePose.CacheBones(Context);

	FBoneContainer& RequiredBones = Context.AnimInstanceProxy->GetRequiredBones();
	if (RequiredBones.IsValid())
	{
		// todo: cache bones if required
		// const FCompactPoseBoneIndex CompactPoseBoneIndex = RequiredBones.MakeCompactPoseIndex();
	}
}

void FAnimNode_RigMapper::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread)
	
	if (IsLODEnabled(Output.AnimInstanceProxy))
	{
		SCOPE_CYCLE_COUNTER(STAT_RigMapperEvaluateNode);
		
		SourcePose.Evaluate(Output);

		if (Alpha > 0.f)
		{
			EvaluateRigMapping(Output);
		}
	}
}

// todo: handle bones
void FAnimNode_RigMapper::EvaluateRigMapping(FPoseContext& Output)
{
	if (RigMapperProcessor.IsValid())
	{
		// Retrieve inputs
		CachedInputValues.Reset(InputCurveMappings.Num());
		CachedInputValues.AddDefaulted(InputCurveMappings.Num());
		UE::Anim::FCurveUtils::BulkGet(Output.Curve, InputCurveMappings, 
	[&Inputs=CachedInputValues](const FRigMapperCurveMapping& InBulkElement, const float InValue)
			{
				Inputs[InBulkElement.CurveIndex] = InValue;
			}
		);
		
		{
			// Evaluate frame
			SCOPE_CYCLE_COUNTER(STAT_RigMapperEvaluateRigs);
			RigMapperProcessor.EvaluateFrame(RigMapperProcessor.GetInputNames(), CachedInputValues, CachedOutputValues);
		}

		{
			SCOPE_CYCLE_COUNTER(STAT_RigMapperSetCurves);

			// Set all output curves for current pose from the given output values.
			UE::Anim::FCurveUtils::BulkSet(Output.Curve, OutputCurveMappings, 
						[&Inputs=CachedInputValues, &Outputs=CachedOutputValues, LerpAlpha=Alpha](const FRigMapperOutputCurveMapping& InBulkElement)
				{
					TOptional<float> Value = Outputs[InBulkElement.CurveIndex];

					// If we have a mapping for a matching input, lerp with the new output using specified alpha
					if (Inputs.IsValidIndex(InBulkElement.InputCurveIndex))
					{
						const TOptional<float> PrevValue = Inputs[InBulkElement.InputCurveIndex];

						if (PrevValue.IsSet())
						{
							if (Value.IsSet())
							{
								Value = FMath::Lerp(PrevValue.GetValue(), Value.GetValue(), LerpAlpha);
							}
							else
							{
								// The value was not set by the rig mapper, do not lerp and keep the previous input value 
								Value = Inputs[InBulkElement.InputCurveIndex];
							}
						}
					}
					// If the value was not set by the rig mapper, and we don't have a matching input, default to 0
					return Value.Get(0.f);
				}
			);
		}
	}
}

int32 FAnimNode_RigMapper::GetLODThreshold() const
{
	return LODThreshold;
}

bool FAnimNode_RigMapper::InitializeRigMapping(USkeletalMesh* TargetMesh)
{
	SCOPE_CYCLE_COUNTER(STAT_RigMapperInitialize);
	
	TArray<URigMapperDefinition*> DefinitionsToLoad;

	// Retrieve the definitions to load and use (either the ones set on the node, or the ones overriden by the SKM asset user data)
	if (TargetMesh)
	{
		LoadedUserData = Cast<URigMapperDefinitionUserData>(TargetMesh->GetAssetUserDataOfClass(URigMapperDefinitionUserData::StaticClass()));
		if (LoadedUserData)
		{
			DefinitionsToLoad = LoadedUserData->Definitions;
		}
	}
	if (DefinitionsToLoad.IsEmpty())
	{
		DefinitionsToLoad = Definitions;
	}

	RigMapperProcessor = FRigMapperProcessor(DefinitionsToLoad);
	if (!RigMapperProcessor.IsValid())
	{
		return false;
	}
	LoadedDefinitions = MoveTemp(DefinitionsToLoad);
	
	// Cache a map of curve indices to bulk get current curve values for the current pose
	const TArray<FName>& InputNames = RigMapperProcessor.GetInputNames();
	InputCurveMappings.Empty();
	InputCurveMappings.Reserve(InputNames.Num());
	for (int32 InputIndex = 0; InputIndex < InputNames.Num(); InputIndex++)
	{
		InputCurveMappings.Add(InputNames[InputIndex], InputIndex);
	}

	// Cache a map of curve indices to bulk set the new curve values for the output pose
	const TArray<FName>& OutputNames = RigMapperProcessor.GetOutputNames();
	OutputCurveMappings.Empty();
	OutputCurveMappings.Reserve(OutputNames.Num());
	for (int32 OutputIndex = 0; OutputIndex < OutputNames.Num(); OutputIndex++)
	{
		const FName& CurveName = OutputNames[OutputIndex];

		// We need to cache the matching input curve index to allow lerping depending on the node's Alpha
		int32 InputIndex = InputNames.Find(CurveName);
		OutputCurveMappings.Add(OutputNames[OutputIndex], OutputIndex, InputIndex);
	}
	
	return true;
}
