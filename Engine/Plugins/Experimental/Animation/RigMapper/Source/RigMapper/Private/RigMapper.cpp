// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigMapper.h"

#include "RigMapperDefinition.h"
#include "RigMapperLog.h"

namespace FacialRigMapping
{
	bool FRigMapper::IsValid() const
	{
		return !Nodes.IsEmpty() && !OutputNodes.IsEmpty() && OutputNodes.Num() == OutputNames.Num() && !InputNodes.IsEmpty() && InputNodes.Num() == InputNames.Num();
	}
	
	void FRigMapper::Reset()
	{
		Nodes.Empty();
    		
    	OutputNodes.Empty();
    	OutputNames.Empty();
    		
    	InputNodes.Empty();
    	InputNames.Empty();
	}

	bool FRigMapper::LoadDefinition(const URigMapperDefinition* Definition)
	{
		Reset();

		const int32 NumNodes = Definition->Inputs.Num()
			+ Definition->Features.Multiply.Num()
			+ Definition->Features.WeightedSums.Num()
			+ Definition->Features.SDKs.Num()
			+ Definition->Features.MathOps.Num();

		Nodes.Reserve(NumNodes);

		InputNodes.Reserve(Definition->Inputs.Num());
		InputNames.Reserve(Definition->Inputs.Num());
		for (int32 Index = 0; Index < Definition->Inputs.Num(); ++Index)
		{
			FName InputName = FName(*Definition->Inputs[Index]);
			FNodePtr Node{ ENodeType::Input, Index};

			InputNodes.Add(Node);
			InputNames.Add(InputName);

			Nodes.Add(InputName, Node);
		}

		for (int32 Index = 0; Index < Definition->Features.Multiply.Num(); ++Index)
		{
			Nodes.Add(FName(*Definition->Features.Multiply[Index].Name), FNodePtr{ENodeType::Multiply, Index});
		}

		for (int32 Index = 0; Index < Definition->Features.WeightedSums.Num(); ++Index)
		{
			Nodes.Add(FName(*Definition->Features.WeightedSums[Index].Name), FNodePtr{ ENodeType::WeightedSum, Index});
		}

		for (int32 Index = 0; Index < Definition->Features.SDKs.Num(); ++Index)
		{
			Nodes.Add(FName(*Definition->Features.SDKs[Index].Name), FNodePtr{ ENodeType::PiecewiseLinear, Index});
		}

		for (int32 Index = 0; Index < Definition->Features.MathOps.Num(); ++Index)
		{
			Nodes.Add(FName(*Definition->Features.MathOps[Index].Name), FNodePtr{ ENodeType::MathOp, Index });
		}

		OutputNodes.Reserve(Definition->Outputs.Num());
		OutputNames.Reserve(Definition->Outputs.Num());
		for (const TPair<FString, FString>& Output : Definition->Outputs)
		{
			const FName OutputName = FName(*Output.Key);
			const FName LinkedNodeName = FName(*Output.Value);

			if (const FNodePtr* Node = Nodes.Find(LinkedNodeName))
			{
				OutputNodes.Add(*Node);
				OutputNames.Add(OutputName);
			}
		}

		if (IsValid())
		{
			NodeCollection.InputNodes.AddDefaulted(Definition->Inputs.Num());

			NodeCollection.MultiplyNodes.AddDefaulted(Definition->Features.Multiply.Num());
			for (int32 Index = 0; Index < NodeCollection.MultiplyNodes.Num(); ++Index)
			{
				const FRigMapperMultiplyFeature& Feature = Definition->Features.Multiply[Index];
				NodeCollection.MultiplyNodes[Index].Initialize(Feature, Nodes);
			}

			NodeCollection.WeightedSumNodes.AddDefaulted(Definition->Features.WeightedSums.Num());
			for (int32 Index = 0; Index < NodeCollection.WeightedSumNodes.Num(); ++Index)
			{
				const FRigMapperWsFeature& Feature = Definition->Features.WeightedSums[Index];
				NodeCollection.WeightedSumNodes[Index].Initialize(Feature, Nodes);
			}

			NodeCollection.PiecewiseLinearNodes.AddDefaulted(Definition->Features.SDKs.Num());
			for (int32 Index = 0; Index < NodeCollection.PiecewiseLinearNodes.Num(); ++Index)
			{
				const FRigMapperSdkFeature& Feature = Definition->Features.SDKs[Index];
				NodeCollection.PiecewiseLinearNodes[Index].Initialize(Feature, Nodes);
			}

			NodeCollection.MathOpNodes.AddDefaulted(Definition->Features.MathOps.Num());
			for (int32 Index = 0; Index < NodeCollection.MathOpNodes.Num(); ++Index)
			{
				const FRigMapperMathFeature& Feature = Definition->Features.MathOps[Index];
				NodeCollection.MathOpNodes[Index].Initialize(Feature, Nodes);
			}
		}

		return IsValid();
	}

	bool FRigMapper::SetDirectValue(const int32 InputIndex, double Value)
	{
		if (InputNodes.IsValidIndex(InputIndex))
		{
			InputNodes[InputIndex].SetDirect(NodeCollection, Value);
			return true;
		}
		return false;
	}
	
	bool FRigMapper::SetDirectValue(const FName& InputName, double Value)
	{
		return SetDirectValue(InputNames.Find(InputName), Value);
	}

	TMap<FName, double> FRigMapper::GetOutputValues(bool bSkipUnset) const
	{
		TMap<FName, double> OutValues;
		OutValues.Reserve(OutputNodes.Num());
	
		for (int32 NodeIndex = 0; NodeIndex < OutputNodes.Num(); NodeIndex++)
		{
			double OutputValue = 0.0f;
			if (OutputNodes[NodeIndex].TryGetValue(NodeCollection, OutputValue) || !bSkipUnset)
			{
				OutValues.Add(OutputNames[NodeIndex], OutputValue);
			}
		}

		return OutValues;
	}

	const TArray<FName>& FRigMapper::GetOutputNames() const
	{
		return OutputNames;
	}

	void FRigMapper::GetOutputValuesInOrder(TArray<double>& OutValues) const
	{
		if (OutValues.Num() != OutputNodes.Num())
		{
			OutValues.Reset(OutputNodes.Num());
			OutValues.AddZeroed(OutputNodes.Num());
		}

		for (int32 NodeIndex = 0; NodeIndex < OutputNodes.Num(); NodeIndex++)
		{
			OutValues[NodeIndex] = OutputNodes[NodeIndex].GetValue(NodeCollection);
		}
	}

	void FRigMapper::GetOptionalOutputValuesInOrder(TArray<TOptional<double>>& OutValues) const
	{
		if (OutValues.Num() != OutputNodes.Num())
		{
			OutValues.Reset(OutputNodes.Num());
			OutValues.AddDefaulted(OutputNodes.Num());
		}

		for (int32 NodeIndex = 0; NodeIndex < OutputNodes.Num(); NodeIndex++)
		{
			double OutputValue;

			if (OutputNodes[NodeIndex].TryGetValue(NodeCollection, OutputValue))
			{
				OutValues[NodeIndex] = OutputValue;
			}
			else
			{
				OutValues[NodeIndex].Reset();
			}
		}
	}

	void FRigMapper::GetOptionalFloatOutputValuesInOrder(TArray<TOptional<float>>& OutValues) const
	{
		if (OutValues.Num() != OutputNodes.Num())
		{
			OutValues.Reset(OutputNodes.Num());
			OutValues.AddDefaulted(OutputNodes.Num());
		}

		for (int32 NodeIndex = 0; NodeIndex < OutputNodes.Num(); NodeIndex++)
		{
			double OutputValue;

			if (OutputNodes[NodeIndex].TryGetValue(NodeCollection, OutputValue))
			{
				OutValues[NodeIndex] = OutputValue;
			}
			else
			{
				OutValues[NodeIndex].Reset();
			}
		}
	}

	void FRigMapper::SetDirty()
	{
		for (TPair<FName, FNodePtr>& Node : Nodes)
		{
			Node.Value.Reset(NodeCollection);
		}
	}

	const TArray<FName>& FRigMapper::GetInputNames() const
	{
		return InputNames;
	}

	////////////////////////////////////////////////////////////////////////////////////
	// FNodePtr
	////////////////////////////////////////////////////////////////////////////////////

	FNodePtr::FNodePtr(ENodeType InNodeType, int32 InDataIndex)
		: DataIndex(InDataIndex)
		, NodeType(InNodeType)
	{
	}

	bool FNodePtr::TryGetValue(const FNodeCollection& NodeCollection, double& OutValue) const
	{
		switch (NodeType)
		{
		case ENodeType::Input:
			return NodeCollection.InputNodes[DataIndex].TryGetValue(NodeCollection, OutValue);
		case ENodeType::WeightedSum:
			return NodeCollection.WeightedSumNodes[DataIndex].TryGetValue(NodeCollection, OutValue);
		case ENodeType::PiecewiseLinear:
			return NodeCollection.PiecewiseLinearNodes[DataIndex].TryGetValue(NodeCollection, OutValue);
		case ENodeType::Multiply:
			return NodeCollection.MultiplyNodes[DataIndex].TryGetValue(NodeCollection, OutValue);
		case ENodeType::MathOp:
			return NodeCollection.MathOpNodes[DataIndex].TryGetValue(NodeCollection, OutValue);
		}
		return false;
	}

	bool FNodePtr::IsInitialized(const FNodeCollection& NodeCollection) const
	{
		switch (NodeType)
		{
		case ENodeType::Input:
			return true;
		case ENodeType::WeightedSum:
			return NodeCollection.WeightedSumNodes[DataIndex].IsInitialized();
		case ENodeType::PiecewiseLinear:
			return NodeCollection.PiecewiseLinearNodes[DataIndex].IsInitialized();
		case ENodeType::Multiply:
			return NodeCollection.MultiplyNodes[DataIndex].IsInitialized();
		case ENodeType::MathOp:
			return NodeCollection.MathOpNodes[DataIndex].IsInitialized();
		}
		return false;
	}

	double FNodePtr::GetValue(const FNodeCollection& NodeCollection) const
	{
		double Value = 0.0;
		TryGetValue(NodeCollection, Value);
		return Value;
	}

	void FNodePtr::SetDirect(FNodeCollection& NodeCollection, double Value)
	{
		switch (NodeType)
		{
		case ENodeType::Input:
			return NodeCollection.InputNodes[DataIndex].SetDirect(Value);
		case ENodeType::WeightedSum:
			return NodeCollection.WeightedSumNodes[DataIndex].SetDirect(Value);
		case ENodeType::PiecewiseLinear:
			return NodeCollection.PiecewiseLinearNodes[DataIndex].SetDirect(Value);
		case ENodeType::Multiply:
			return NodeCollection.MultiplyNodes[DataIndex].SetDirect(Value);
		case ENodeType::MathOp:
			return NodeCollection.MathOpNodes[DataIndex].SetDirect(Value);
		}
	}

	void FNodePtr::Reset(FNodeCollection& NodeCollection)
	{
		switch (NodeType)
		{
		case ENodeType::Input:
			return NodeCollection.InputNodes[DataIndex].Reset();
		case ENodeType::WeightedSum:
			return NodeCollection.WeightedSumNodes[DataIndex].Reset();
		case ENodeType::PiecewiseLinear:
			return NodeCollection.PiecewiseLinearNodes[DataIndex].Reset();
		case ENodeType::Multiply:
			return NodeCollection.MultiplyNodes[DataIndex].Reset();
		case ENodeType::MathOp:
			return NodeCollection.MathOpNodes[DataIndex].Reset();
		}
	}

	////////////////////////////////////////////////////////////////////////////////////
	// FEvalNodeWeightedSum
	////////////////////////////////////////////////////////////////////////////////////

	void FEvalNodeWeightedSum::Initialize(const FRigMapperWsFeature& FeatureDefinition, const TMap<FName, FNodePtr>& Nodes)
	{
		if (FeatureDefinition.Range.bHasLowerBound)
		{
			Range.SetLowerBound(TRange<double>::BoundsType(FeatureDefinition.Range.LowerBound));
		}
		if (FeatureDefinition.Range.bHasUpperBound)
		{
			Range.SetUpperBound(TRange<double>::BoundsType(FeatureDefinition.Range.UpperBound));
		}

		for (const TPair<FString, double>& Pair : FeatureDefinition.Inputs)
		{
			if (const FNodePtr* Node = Nodes.Find(FName(*Pair.Key)))
			{
				WeightedLinkedInputs.Emplace(*Node, Pair.Value);
			}
		}

		// Will be considered not initialized if any node could not be found
		bInitialized = WeightedLinkedInputs.Num() == FeatureDefinition.Inputs.Num();
	}

	bool FEvalNodeWeightedSum::Evaluate(const FNodeCollection& NodeCollection, double& OutValue) const
	{
		bool bInputEvaluated = false;
		OutValue = 0;

		for (const TPair<FNodePtr, double>& WeightedInput : WeightedLinkedInputs)
		{
			double InputValue = 0.0f;

			if (WeightedInput.Key.TryGetValue(NodeCollection, InputValue))
			{
				bInputEvaluated = true;
			}
			OutValue += InputValue * WeightedInput.Value;
		}

		if (Range.HasLowerBound() && OutValue < Range.GetLowerBoundValue())
		{
			OutValue = Range.GetLowerBoundValue();
		}
		if (Range.HasUpperBound() && OutValue > Range.GetUpperBoundValue())
		{
			OutValue = Range.GetUpperBoundValue();
		}

		return bInputEvaluated;
	}

	////////////////////////////////////////////////////////////////////////////////////
	// FEvalNodePiecewiseLinear
	////////////////////////////////////////////////////////////////////////////////////

	void FEvalNodePiecewiseLinear::Initialize(const FRigMapperSdkFeature& FeatureDefinition, const TMap<FName, FNodePtr>& Nodes)
	{
		LinkedInput = FNodePtr{};

		if (const FNodePtr* Node = Nodes.Find(FName(*FeatureDefinition.Input)))
		{
			LinkedInput = *Node;
		}

		Keys.Reserve(FeatureDefinition.Keys.Num());
		for (const FRigMapperSdkKey& Key : FeatureDefinition.Keys)
		{
			Keys.Emplace(Key.In, Key.Out);
		}
		Keys.Sort([](const TPair<double, double>& Key1, const TPair<double, double>& Key2) { return Key1.Key < Key2.Key; });

		bInitialized = LinkedInput.IsValid();
	}

	bool FEvalNodePiecewiseLinear::Evaluate(const FNodeCollection& NodeCollection, double& OutValue) const
	{
		double InputValue = 0.0f;

		if (!LinkedInput.TryGetValue(NodeCollection, InputValue))
		{
			return false;
		}

		if (!Evaluate_Static(InputValue, Keys, OutValue))
		{
			UE_LOGF(LogRigMapper, Warning, "PiecewiseLinear could not calculate the output value")
				return false;
		}

		return true;
	}

	bool FEvalNodePiecewiseLinear::Evaluate_Static(const double InputValue, TConstArrayView<TPair<double, double>> Keys, double& OutValue)
	{
		if (InputValue <= Keys[0].Key)
		{
			OutValue = Keys[0].Value;
			return true;
		}

		if (InputValue >= Keys.Last().Key)
		{
			OutValue = Keys.Last().Value;
			return true;
		}

		for (int32 KeyIndex = 1; KeyIndex < Keys.Num(); KeyIndex++)
		{
			const TPair<double, double>& CurrentKey = Keys[KeyIndex];

			if (InputValue == CurrentKey.Key)
			{
				OutValue = CurrentKey.Value;
				return true;
			}
			if (InputValue < CurrentKey.Key)
			{
				const TPair<double, double>& PrevKey = Keys[KeyIndex - 1];

				const double Percent = (InputValue - PrevKey.Key) / (CurrentKey.Key - PrevKey.Key);

				OutValue = PrevKey.Value + Percent * (CurrentKey.Value - PrevKey.Value);
				return true;
			}
		}

		return false;
	}

	////////////////////////////////////////////////////////////////////////////////////
	// FEvalNodeMultiply
	////////////////////////////////////////////////////////////////////////////////////

	void FEvalNodeMultiply::Initialize(const FRigMapperMultiplyFeature& FeatureDefinition, const TMap<FName, FNodePtr>& Nodes)
	{
		LinkedInputs.Empty(FeatureDefinition.Inputs.Num());

		for (const FString& InputName : FeatureDefinition.Inputs)
		{
			if (const FNodePtr* Node = Nodes.Find(FName(*InputName)))
			{
				if (Node->IsValid())
				{
					LinkedInputs.Add(*Node);
				}
			}
		}

		// Will be considered not initialized if any node could not be found
		bInitialized = LinkedInputs.Num() == FeatureDefinition.Inputs.Num();
	}

	bool FEvalNodeMultiply::Evaluate(const FNodeCollection& NodeCollection, double& OutValue) const
	{
		if (LinkedInputs.IsEmpty())
		{
			return false;
		}

		OutValue = 1.0;
		bool bInputEvaluated = false;

		for (const FNodePtr& Input : LinkedInputs)
		{
			double InputValue = 0.0f;

			if (Input.TryGetValue(NodeCollection, InputValue))
			{
				bInputEvaluated = true;
			}
			OutValue *= InputValue;
		}

		return bInputEvaluated;
	}

	////////////////////////////////////////////////////////////////////////////////////
	// FEvalNodeMathOp
	////////////////////////////////////////////////////////////////////////////////////

	void FEvalNodeMathOp::Initialize(const FRigMapperMathFeature& FeatureDefinition, const TMap<FName, FNodePtr>& Nodes)
	{
		Operation = FeatureDefinition.Operation;
		LinkedInputs.Reserve(FeatureDefinition.Inputs.Num());

		bInitialized = true;
		for (int32 i = 0; i < FeatureDefinition.Inputs.Num(); ++i)
		{
			FMathInput& Input = LinkedInputs.AddDefaulted_GetRef();

			const FString& NodeName = FeatureDefinition.Inputs[i].NodeName;
			if (!NodeName.IsEmpty())
			{
				if (const FNodePtr* FoundNode = Nodes.Find(FName(*NodeName)))
				{
					Input.Node = *FoundNode;
				}
				else
				{
					// Will be considered not initialized if any named node could not be found
					bInitialized = false;
					break;
				}
			}
			
			Input.ConstantValue = FeatureDefinition.Inputs[i].ConstantValue;
		}
	}

	bool FEvalNodeMathOp::Evaluate(const FNodeCollection& NodeCollection, double& OutValue) const
	{
		auto GetInputValue = [this, &NodeCollection](int32 Index, double& Value) -> bool
			{
				const FMathInput& Input = LinkedInputs[Index];
			
				if (Input.Node.IsValid())
				{
					return Input.Node.TryGetValue(NodeCollection, Value);
				}
				// Node not connected, use constant value
				Value = Input.ConstantValue;
				return true;
			};

		if (LinkedInputs.IsEmpty()) return false;

		double First = 0.0;
		if (!GetInputValue(0, First)) return false;

		switch (Operation)
		{
		case ERigMapperMathOperation::Abs:      OutValue = FMath::Abs(First); break;
		case ERigMapperMathOperation::Negate:   OutValue = -First; break;
		case ERigMapperMathOperation::Floor:    OutValue = FMath::FloorToDouble(First); break;
		case ERigMapperMathOperation::Ceil:     OutValue = FMath::CeilToDouble(First); break;
		case ERigMapperMathOperation::Round:    OutValue = FMath::RoundToDouble(First); break;
		case ERigMapperMathOperation::Min:
		{
			OutValue = First;
			for (int32 i = 1; i < LinkedInputs.Num(); ++i)
			{
				double Val; 
				if (!GetInputValue(i, Val)) return false;
				OutValue = FMath::Min(OutValue, Val);
			}
			break;
		}
		case ERigMapperMathOperation::Max:
		{
			OutValue = First;
			for (int32 i = 1; i < LinkedInputs.Num(); ++i)
			{
				double Val;
				if (!GetInputValue(i, Val)) return false;
				OutValue = FMath::Max(OutValue, Val);
			}
			break;
		}
		case ERigMapperMathOperation::Average:
		{
			double Sum = First;
			for (int32 i = 1; i < LinkedInputs.Num(); ++i)
			{
				double Val;
				if (!GetInputValue(i, Val)) return false;
				Sum += Val;
			}
			OutValue = Sum / LinkedInputs.Num();
			break;
		}
		case ERigMapperMathOperation::Multiply:
		{
			double Product = First;
			for (int32 i = 1; i < LinkedInputs.Num(); ++i)
			{
				if (FMath::IsNearlyZero(Product))
				{
					break;
				}
				double Val;
				if (!GetInputValue(i, Val))	return false;
				Product *= Val;
			}
			OutValue = Product;
			break;
		}
		case ERigMapperMathOperation::Divide:
		{
			double Quotient = First;
			for (int32 i = 1; i < LinkedInputs.Num(); ++i)
			{
				if (FMath::IsNearlyZero(Quotient)) break;
				double Divisor;
				if (!GetInputValue(i, Divisor)) return false;
				Quotient = FMath::IsNearlyZero(Divisor) ? 0.0 : Quotient / Divisor;
			}
			OutValue = Quotient;
			break;
		}
		case ERigMapperMathOperation::Sum:
		{
			double Sum = First;
			for (int32 i = 1; i < LinkedInputs.Num(); ++i)
			{
				double Val;
				if (!GetInputValue(i, Val))	return false;
				Sum += Val;
			}
			OutValue = Sum;
			break;
		}
		case ERigMapperMathOperation::Clamp:
		{
			double MinVal, MaxVal;
			if (LinkedInputs.Num() < 3)
			{
				return false;
			}
			if (!GetInputValue(1, MinVal))
			{
				return false;
			}
			if (!GetInputValue(2, MaxVal))
			{
				return false;
			}
			OutValue = FMath::Clamp(First, MinVal, MaxVal);
			break;
		}
		case ERigMapperMathOperation::Lerp:
		{
			double B, Alpha;
			if (LinkedInputs.Num() < 3)
			{
				return false;
			}
			if (!GetInputValue(1, B))
			{
				return false;
			}
			if (!GetInputValue(2, Alpha))
			{
				return false;
			}
			OutValue = FMath::Lerp(First, B, Alpha);
			break;
		}
		default:
			OutValue = First;
			return false;
		}
		return true;
	}
}