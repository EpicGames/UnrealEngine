// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigMapperDefinition.h"

#include "RigMapper.h"
#include "RigMapperProcessor.h"
#include "Misc/FileHelper.h"
#include "Serialization/JsonTypes.h"
#include "Dom/JsonValue.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonSerializer.h"
#include "RigMapperLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigMapperDefinition)

bool FRigMapperFeature::IsValid(const TArray<FString>& InputNames, bool bWarn, FRigMapperValidationContext* OutContext) const
{
	bool bValid = true;
	
	if (Name.IsEmpty())
	{
		const FString Message = TEXT("Invalid (empty) feature name");
		if (OutContext)
		{
			OutContext->AddError(Message);
		}
		else
		{
			UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
		}
		bValid = false;
	}

	if (GetFeatureType() != ERigMapperFeatureType::Input && GetFeatureType() != ERigMapperFeatureType::NullOutput)
	{
		TArray<FString> FeatureInputs;
		GetInputs(FeatureInputs);

		if (FeatureInputs.Contains(Name))
		{
			const FString Message = FString::Printf(TEXT("Feature %ls is referencing itself"), *Name);
			if (OutContext)
			{
				OutContext->AddError(Message);
			}
			else
			{
				UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
			}
			bValid = false;
		}

		if (FeatureInputs.IsEmpty())
		{
			const FString Message = FString::Printf(TEXT("Feature %ls does not reference any input"), *Name);
			if (OutContext)
			{
				OutContext->AddError(Message);
			}
			else
			{
				UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
			}
			bValid = false;
		}

		for (const FString& Input : FeatureInputs)
		{
			if (GetFeatureType() == ERigMapperFeatureType::MathOp && Input.IsEmpty())
			{
				continue; // MathOp is allowed to have empty inputs
			}
			if (!InputNames.Contains(Input))
			{
				const FString Message = FString::Printf(TEXT("Undefined input or feature %ls referenced in feature %ls"), *Input, *Name);
				if (OutContext)
				{
					OutContext->AddError(Message);
				}
				else
				{
					UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
				}
				bValid = false;
			}
		}
	}
	return bValid;
}

bool FRigMapperFeature::GetJsonArray(TSharedPtr<FJsonObject> JsonObject, TArray<TSharedPtr<FJsonValue>>& OutArray, const FString& Identifier, const FString& OwnerIdentifier) const
{
	if (!OwnerIdentifier.IsEmpty())
	{
		if (!JsonObject->HasTypedField<EJson::Object>(*OwnerIdentifier))
		{
			UE_LOGF(LogRigMapper, Warning, "Missing '%ls' field for feature %ls", *OwnerIdentifier, *Name)
			return false;
		}
		JsonObject = JsonObject->GetObjectField(OwnerIdentifier);
	}
	if (!JsonObject->HasTypedField<EJson::Array>(*Identifier))
	{
		UE_LOGF(LogRigMapper, Warning, "Missing '%ls' field for feature %ls", *Identifier, *Name)
		return false;
	}
	OutArray = JsonObject->GetArrayField(*Identifier);
	return true;
}

bool FRigMapperMultiplyFeature::LoadFromJsonObject(const TSharedPtr<FJsonObject>& JsonObject)
{
	TArray<TSharedPtr<FJsonValue>> InputFeatures;
	
	if (!GetJsonArray(JsonObject, InputFeatures, TEXT("input_features")))
	{
		return false;
	}
	if (InputFeatures.Num() < 2)
	{
		UE_LOGF(LogRigMapper, Warning, "Feature %ls does not reference enough input", *Name)
		return false;
	}
	for (const auto& InputFeature : InputFeatures)
	{
		Inputs.Add(InputFeature->AsString());
	}
	
	return true;
}

bool FRigMapperMultiplyFeature::IsValid(const TArray<FString>& InputNames, bool bWarn, FRigMapperValidationContext* OutContext) const
{
	bool bValid = FRigMapperFeature::IsValid(InputNames, bWarn, OutContext);
	
	if (bValid && Inputs.Num() < 2)
	{
		const FString Message = FString::Printf(TEXT("Feature %ls does not reference enough input"), *Name);
		if (OutContext)
		{
			OutContext->AddError(Message);
		}
		else
		{
			UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
		}
		return false;
	}
	
	return bValid;
}

bool FRigMapperWsFeature::LoadFromJsonObject(const TSharedPtr<FJsonObject>& JsonObject)
{
	TArray<TSharedPtr<FJsonValue>> JsonInputs;
	if (!GetJsonArray(JsonObject, JsonInputs, TEXT("input_features")))
	{
		return false;
	}
	
	TArray<TSharedPtr<FJsonValue>> JsonWeights;
	if (!GetJsonArray(JsonObject, JsonWeights, TEXT("weights"), TEXT("params")))
	{
		return false;
	}
	if (JsonWeights.Num() != JsonInputs.Num())
	{
		UE_LOGF(LogRigMapper, Warning, "Number of inputs does not match number of weights for feature %ls", *Name)
		return false;
	}
	for (int32 InputIndex = 0; InputIndex < JsonInputs.Num(); InputIndex++)
	{
		Inputs.Add(JsonInputs[InputIndex]->AsString(), JsonWeights[InputIndex]->AsNumber());
	}

	Range.bHasLowerBound = JsonObject->GetObjectField(TEXT("params"))->HasField(TEXT("min"));
	if (Range.bHasLowerBound)
	{
		Range.LowerBound = JsonObject->GetObjectField(TEXT("params"))->GetNumberField(TEXT("min"));
	}
	Range.bHasUpperBound = JsonObject->GetObjectField(TEXT("params"))->HasField(TEXT("max"));
	if (Range.bHasUpperBound)
	{
		Range.UpperBound = JsonObject->GetObjectField(TEXT("params"))->GetNumberField(TEXT("max"));
	}
	if (Range.bHasLowerBound && Range.bHasUpperBound && Range.LowerBound > Range.UpperBound)
	{
		UE_LOGF(LogRigMapper, Warning, "Invalid range for feature %ls", *Name)
		return false;
	}
	return true;
}

bool FRigMapperWsFeature::IsValid(const TArray<FString>& InputNames, bool bWarn, FRigMapperValidationContext* OutContext) const
{
	bool bValid = FRigMapperFeature::IsValid(InputNames, bWarn, OutContext);

	double TotalWeight = 0;
	for (const TPair<FString, double>& InputWeight : Inputs)
	{
		TotalWeight += InputWeight.Value;
	}
	if (bWarn && TotalWeight == 0)
	{
		const FString Message = FString::Printf(TEXT("Total Weights for feature %ls add up to 0"), *Name);
		if (OutContext)
		{
			OutContext->AddWarning(Message);
		}
		else
		{
			UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
		}
	}
	if (bWarn && !Range.bHasLowerBound && TotalWeight < -1.000001f)
	{
		const FString Message = FString::Printf(TEXT("Total Weights for feature %ls are quite low (%f) even though a lower range bound was not set"), *Name, TotalWeight);
		if (OutContext)
		{
			OutContext->AddWarning(Message);
		}
		else
		{
			UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
		}
	}
	if (bWarn && !Range.bHasUpperBound && TotalWeight > 1.000001f)
	{
		const FString Message = FString::Printf(TEXT("Total Weights for feature %ls are quite high (%f) even though an upper range bound was not set"), *Name, TotalWeight);
		if (OutContext)
		{
			OutContext->AddWarning(Message);
		}
		else
		{
			UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
		}
	}
	if (Range.bHasLowerBound && Range.bHasUpperBound && Range.LowerBound > Range.UpperBound)
	{
		const FString Message = FString::Printf(TEXT("Range of [%f-%f] for feature %ls is invalid"), Range.LowerBound, Range.UpperBound, *Name);
		if (OutContext)
		{
			OutContext->AddError(Message);
		}
		else
		{
			UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
		}
		bValid = false;
	}
	
	return bValid;
}

void FRigMapperWsFeature::GetInputs(TArray<FString>& OutInputs) const
{
	Inputs.GenerateKeyArray(OutInputs);
}

bool FRigMapperSdkFeature::LoadFromJsonObject(const TSharedPtr<FJsonObject>& JsonObject)
{
	TArray<TSharedPtr<FJsonValue>> JsonInputs;
	if (!GetJsonArray(JsonObject, JsonInputs, TEXT("input_features")))
	{
		return false;
	}
	if (JsonInputs.Num() != 1)
	{
		UE_LOGF(LogRigMapper, Warning, "Sdk feature %ls should have a single element in the 'input_features' array field", *Name)
		return false;
	}
	Input = JsonInputs[0]->AsString();

	TArray<TSharedPtr<FJsonValue>> InValues;
	if (!GetJsonArray(JsonObject, InValues, TEXT("in_val"), TEXT("params")))
	{
		return false;
	}

	TArray<TSharedPtr<FJsonValue>> OutValues;
	if (!GetJsonArray(JsonObject, OutValues, TEXT("out_val"), TEXT("params")))
	{
		return false;
	}

	if (InValues.Num() < 2)
	{
		UE_LOGF(LogRigMapper, Warning, "Not enough keys for SDK feature %ls (expected minimum 2, got %d)", *Name, InValues.Num())
	}
	if (InValues.Num() != OutValues.Num())
	{
		UE_LOGF(LogRigMapper, Warning, "Number of input values does not match number of output values for feature %ls", *Name)
		return false;
	}
	for (int32 ValueIndex = 0 ; ValueIndex < InValues.Num(); ValueIndex++)
	{
		Keys.Add({ InValues[ValueIndex]->AsNumber(), OutValues[ValueIndex]->AsNumber() });
	}
	Keys.Sort([](const FRigMapperSdkKey& Key1, const FRigMapperSdkKey Key2) { return Key1.In < Key2.In; });
	
	return true;
}

bool FRigMapperSdkFeature::IsValid(const TArray<FString>& InputNames, bool bWarn, FRigMapperValidationContext* OutContext) const
{
	bool bValid = FRigMapperFeature::IsValid(InputNames, bWarn, OutContext);

	if (Keys.Num() < 2)
	{
		const FString Message = FString::Printf(TEXT("Not enough keys for SDK feature %ls (expected minimum 2, got %d)"), *Name, Keys.Num());
		if (OutContext)
		{
			OutContext->AddError(Message);
		}
		else
		{
			UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
		}
		bValid = false;
	}
	
	return bValid;
}

static const TMap<FString, ERigMapperMathOperation> MathOpStringToEnum = {
	{ TEXT("min"),      ERigMapperMathOperation::Min },
	{ TEXT("max"),      ERigMapperMathOperation::Max },
	{ TEXT("abs"),      ERigMapperMathOperation::Abs },
	{ TEXT("negate"),   ERigMapperMathOperation::Negate },
	{ TEXT("floor"),    ERigMapperMathOperation::Floor },
	{ TEXT("ceil"),     ERigMapperMathOperation::Ceil },
	{ TEXT("round"),    ERigMapperMathOperation::Round },
	{ TEXT("divide"),   ERigMapperMathOperation::Divide },
	{ TEXT("multiply"), ERigMapperMathOperation::Multiply },
	{ TEXT("sum"),      ERigMapperMathOperation::Sum },
	{ TEXT("clamp"),    ERigMapperMathOperation::Clamp },
	{ TEXT("lerp"),     ERigMapperMathOperation::Lerp },
	{ TEXT("average"),  ERigMapperMathOperation::Average },
};

static const TMap<ERigMapperMathOperation, FString> MathOpEnumToString = {
	{ ERigMapperMathOperation::Min,      TEXT("min") },
	{ ERigMapperMathOperation::Max,      TEXT("max") },
	{ ERigMapperMathOperation::Abs,      TEXT("abs") },
	{ ERigMapperMathOperation::Negate,   TEXT("negate") },
	{ ERigMapperMathOperation::Floor,    TEXT("floor") },
	{ ERigMapperMathOperation::Ceil,     TEXT("ceil") },
	{ ERigMapperMathOperation::Round,    TEXT("round") },
	{ ERigMapperMathOperation::Divide,   TEXT("divide") },
	{ ERigMapperMathOperation::Multiply, TEXT("multiply") },
	{ ERigMapperMathOperation::Sum,      TEXT("sum") },
	{ ERigMapperMathOperation::Clamp,    TEXT("clamp") },
	{ ERigMapperMathOperation::Lerp,     TEXT("lerp") },
	{ ERigMapperMathOperation::Average,  TEXT("average") },
};

bool FRigMapperMathFeature::LoadFromJsonObject(const TSharedPtr<FJsonObject>& JsonObject)
{
	const FString OperationStr = JsonObject->GetStringField(TEXT("operation"));
	if (const ERigMapperMathOperation* FoundOp = MathOpStringToEnum.Find(OperationStr))
	{
		Operation = *FoundOp;
	}
	else
	{
		UE_LOGF(LogRigMapper, Warning, "Unknown math operation '%ls' for feature %ls", *OperationStr, *Name);
		return false;
	}

	TArray<TSharedPtr<FJsonValue>> JsonInputs;
	if (!GetJsonArray(JsonObject, JsonInputs, TEXT("input_features")))
	{
		return false;
	}

	TArray<TSharedPtr<FJsonValue>> JsonConstants;
	if (!GetJsonArray(JsonObject, JsonConstants, TEXT("constants"), TEXT("params")))
	{
		return false;
	}
	if (JsonConstants.Num() != JsonInputs.Num())
	{
		UE_LOGF(LogRigMapper, Warning, "Number of inputs does not match number of constant values for math feature %ls", *Name)
			return false;
	}
	for (int32 InputIndex = 0; InputIndex < JsonInputs.Num(); InputIndex++)
	{
		FRigMapperMathInput& Input = Inputs.AddDefaulted_GetRef();
		Input.NodeName = JsonInputs[InputIndex]->AsString();
		Input.ConstantValue = JsonConstants[InputIndex]->AsNumber();
	}
	return true;
}


bool FRigMapperMathFeature::IsValid(const TArray<FString>& InputNames, bool bWarn, FRigMapperValidationContext* OutContext) const
{
	bool bValid = FRigMapperFeature::IsValid(InputNames, bWarn, OutContext);

	const int32 MinInputs = GetMinInputCount(Operation);
	const int32 MaxInputs = GetMaxInputCount(Operation);

	if (Inputs.Num() < MinInputs)
	{
		const FString Message = FString::Printf(TEXT("MathOp %s requires at least %d input(s) but has %d"), *Name, MinInputs, Inputs.Num());
		if (OutContext)
		{
			OutContext->AddError(Message);
		}
		else
		{
			UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
		}
		bValid = false;
	}
	else if (Inputs.Num() > MaxInputs)
	{
		const FString Message = FString::Printf(TEXT("MathOp %s accepts at most %d input(s) but has %d. Extra inputs will be ignored"), *Name, MaxInputs, Inputs.Num());
		if (OutContext)
		{
			OutContext->AddWarning(Message);
		}
		else if (bWarn)
		{
			UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
		}
	}
	return bValid;
}

void FRigMapperMathFeature::GetInputs(TArray<FString>& OutInputs) const
{
	// Get only named nodes as inputs
	OutInputs.Reset();
	for (const FRigMapperMathInput& Input : Inputs)
	{
		OutInputs.Add(Input.NodeName);
	}
}

bool FRigMapperMathFeature::BakeInput(URigMapperLinkedDefinitions* LinkedDefinitions, int32 DefinitionIndex, FBakedInput& OutBakedInput, FRigMapperValidationContext* OutContext) const
{
	const FString BakedInputName = FString::Printf(TEXT("%s:%d"), *Name, DefinitionIndex);

	const TSharedPtr<FRigMapperMathFeature> BakedMathFeature = MakeShared<FRigMapperMathFeature>(BakedInputName);
	OutBakedInput.Key = BakedMathFeature;
	OutBakedInput.Value.Empty();
	bool bMissingIsNotNullOutput = false;

	for (const FRigMapperMathInput& FeatureInput : Inputs)
	{
		if (FeatureInput.NodeName.IsEmpty())
		{
			// Constant-only input — preserve in baked output with its value
			BakedMathFeature->Inputs.Add(FRigMapperMathInput(TEXT(""), FeatureInput.ConstantValue));
			continue;
		}

		FBakedInput SubFeatureBakedInput;
		if (!LinkedDefinitions->GetBakedInputRec(FeatureInput.NodeName, DefinitionIndex, SubFeatureBakedInput, bMissingIsNotNullOutput, OutContext))
		{
			continue;
		}

		if (SubFeatureBakedInput.Key->GetFeatureType() == ERigMapperFeatureType::MathOp)
		{
			const FRigMapperMathFeature* SubFeatureMathBakedInput = static_cast<FRigMapperMathFeature*>(SubFeatureBakedInput.Key.Get());
			BakedMathFeature->Inputs.Append(SubFeatureMathBakedInput->Inputs);
		}
		else
		{
			FRigMapperMathInput& NewInput = BakedMathFeature->Inputs.AddDefaulted_GetRef();
			NewInput.NodeName = SubFeatureBakedInput.Key->Name;
			OutBakedInput.Value.Add(SubFeatureBakedInput.Key);
		}
		OutBakedInput.Value.Append(SubFeatureBakedInput.Value);
	}

	return OutBakedInput.Key.IsValid() && !bMissingIsNotNullOutput; // if we have a missing definition which is not defined as a NullOutput, flag a failure
}

bool FRigMapperMathFeature::ContainsInput(const FString& InInput) const
{
	return Inputs.ContainsByPredicate([InInput](const FRigMapperMathInput& Other) { return Other.NodeName == InInput; });
}

bool FRigMapperFeatureDefinitions::AddFromJsonObject(const FString& FeatureName, const TSharedPtr<FJsonObject>& JsonObject)
{
	if (!JsonObject->HasTypedField<EJson::String>(TEXT("type")))
	{
		UE_LOGF(LogRigMapper, Warning, "Missing 'type' field for feature %ls", *FeatureName)
		return false;
	}
	const FString FeatureType = JsonObject->GetStringField(TEXT("type"));

	bool bValidFeature = false;
	
	if (FeatureType == "weighted_sum")
	{
		WeightedSums.Add(FRigMapperWsFeature(FeatureName));
		bValidFeature = WeightedSums.Last().LoadFromJsonObject(JsonObject);
	}
	else if (FeatureType == "sdk")
	{
		SDKs.Add(FRigMapperSdkFeature(FeatureName));
		bValidFeature = SDKs.Last().LoadFromJsonObject(JsonObject);
	}
	else if (FeatureType == "multiply")
	{
		Multiply.Add(FRigMapperMultiplyFeature(FeatureName));
		bValidFeature = Multiply.Last().LoadFromJsonObject(JsonObject);
	}
	else if (FeatureType == "math_op")
	{
		MathOps.Add(FRigMapperMathFeature(FeatureName));
		bValidFeature = MathOps.Last().LoadFromJsonObject(JsonObject);
	}
	else
	{
		UE_LOGF(LogRigMapper, Warning, "Invalid type for feature %ls (%ls)", *FeatureName, *FeatureType)
		return bValidFeature;
	}
	return bValidFeature;
}

bool FRigMapperFeatureDefinitions::GetFeatureNames(TArray<FString>& OutFeatureNames) const
{
	bool bFoundDuplicate = false;

	// todo: from interface
	for (const FRigMapperMultiplyFeature& Feature : Multiply)
	{
		bFoundDuplicate |= OutFeatureNames.Contains(Feature.Name);
		OutFeatureNames.Add(Feature.Name);
	}
	for (const FRigMapperSdkFeature& Feature : SDKs)
	{
		bFoundDuplicate |= OutFeatureNames.Contains(Feature.Name);
		OutFeatureNames.Add(Feature.Name);
	}
	for (const FRigMapperWsFeature& Feature : WeightedSums)
	{
		bFoundDuplicate |= OutFeatureNames.Contains(Feature.Name);
		OutFeatureNames.Add(Feature.Name);
	}
	for (const FRigMapperMathFeature& Feature : MathOps)
	{
		bFoundDuplicate |= OutFeatureNames.Contains(Feature.Name);
		OutFeatureNames.Add(Feature.Name);
	}
	return !bFoundDuplicate;
}

bool FRigMapperFeatureDefinitions::IsValid(const TArray<FString>& InputNames, bool bWarn, FRigMapperValidationContext* OutContext) const
{
	TArray<FString> FeatureNames;
	bool bValid = true;

	const bool bDuplicateFeatureName = GetFeatureNames(FeatureNames);
	TArray<FString> CheckFeatureNames;
	for (const FString& FeatureName : FeatureNames)
	{
		if (InputNames.Contains(FeatureName))
		{
			const FString Message = FString::Printf(TEXT("Conflicting input and feature name: %ls"), *FeatureName);
			if (OutContext)
			{
				OutContext->AddError(Message);
			}
			else
			{
				UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
			}
			bValid = false;
		}
		if (bDuplicateFeatureName)
		{
			if (CheckFeatureNames.Contains(FeatureName))
			{
				const FString Message = FString::Printf(TEXT("Duplicate feature name: %ls"), *FeatureName);
				if (OutContext)
				{
					OutContext->AddError(Message);
				}
				else
				{
					UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
				}
				bValid = false;
			}
			else
			{
				CheckFeatureNames.Add(FeatureName);
			}
		}
	}

	TArray<FString> FeatureAndInputNames = MoveTemp(FeatureNames);
	FeatureAndInputNames.Append(InputNames);

	for (const FRigMapperMultiplyFeature& Feature : Multiply)
	{
		bValid &= Feature.IsValid(FeatureAndInputNames, bWarn, OutContext);
	}
	for (const FRigMapperSdkFeature& Feature : SDKs)
	{
		bValid &= Feature.IsValid(FeatureAndInputNames, bWarn, OutContext);
	}
	for (const FRigMapperWsFeature& Feature : WeightedSums)
	{
		bValid &= Feature.IsValid(FeatureAndInputNames, bWarn, OutContext);
	}
	for (const FRigMapperMathFeature& Feature : MathOps)
	{
		bValid &= Feature.IsValid(FeatureAndInputNames, bWarn, OutContext);
	}
	
	return bValid;
}

void FRigMapperFeatureDefinitions::Empty()
{
	Multiply.Empty();
	WeightedSums.Empty();
	SDKs.Empty();
	MathOps.Empty();
}

FRigMapperFeature* FRigMapperFeatureDefinitions::Find(const FString& FeatureName, ERigMapperFeatureType& OutFeatureType)
{
	if (FRigMapperMultiplyFeature* Feature = Multiply.FindByPredicate([FeatureName](const FRigMapperMultiplyFeature& Other) { return Other.Name == FeatureName; }))
	{
		OutFeatureType = ERigMapperFeatureType::Multiply;
		return Feature;
	}

	if (FRigMapperWsFeature* Feature = WeightedSums.FindByPredicate([FeatureName](const FRigMapperWsFeature& Other) { return Other.Name == FeatureName; }))
	{
		OutFeatureType = ERigMapperFeatureType::WeightedSum;
		return Feature;
	}

	if (FRigMapperSdkFeature* Feature = SDKs.FindByPredicate([FeatureName](const FRigMapperSdkFeature& Other) { return Other.Name == FeatureName; }))
	{
		OutFeatureType = ERigMapperFeatureType::SDK;
		return Feature;
	}

	if (FRigMapperMathFeature* Feature = MathOps.FindByPredicate([FeatureName](const FRigMapperMathFeature& Other) { return Other.Name == FeatureName; }))
	{
		OutFeatureType = ERigMapperFeatureType::MathOp;
		return Feature;
	}

	return nullptr;
}

bool URigMapperDefinition::LoadFromJsonFile(const FFilePath& JsonFilePath)
{
	FString JsonAsString;
		
	if (!FFileHelper::LoadFileToString(JsonAsString, *JsonFilePath.FilePath))
	{
		UE_LOGF(LogRigMapper, Warning, "Could not open json file")
		return false;
	}

	return LoadFromJsonString(JsonAsString);
}

void URigMapperDefinition::Empty()
{
	SetDefinitionValid(false);
	Inputs.Empty();
	Outputs.Empty();
	Features.Empty();
	NullOutputs.Empty();
}

bool URigMapperDefinition::LoadFromJsonString(const FString& JsonString)
{
	TSharedPtr<FJsonObject> JsonObject = MakeShareable(new FJsonObject());
	const TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(JsonString);
	
	if (!FJsonSerializer::Deserialize(JsonReader, JsonObject) && JsonObject.IsValid())
	{
		UE_LOGF(LogRigMapper, Warning, "Could not deserialize json data to load definition")
		return false;
	}

	Empty();
	SetDefinitionValid(true);
	
	TArray<FString> FeatureNames;

	const bool bValidInputs = LoadInputsFromJsonObject(JsonObject);
	if (!bValidInputs)
	{
		SetDefinitionValid(false);
	}
	const bool bValidFeatures = LoadFeaturesFromJsonObject(JsonObject, FeatureNames);
	if (!bValidFeatures)
	{
		SetDefinitionValid(false);
	}
	const bool bValidOutputs = LoadOutputsFromJsonObject(JsonObject, FeatureNames);
	if (!bValidOutputs)
	{
		SetDefinitionValid(false);
	}
	const bool bValidNullOutputs = LoadNullOutputsFromJsonObject(JsonObject);
	if (!bValidNullOutputs)
	{
		SetDefinitionValid(false);
	}

	OnRigMapperDefinitionCreated.Broadcast();

	return bValidated;
}

bool URigMapperDefinition::LoadInputsFromJsonObject(const TSharedPtr<FJsonObject>& JsonObject)
{
	if (!JsonObject->HasTypedField(TEXT("inputs"), EJson::Array))
	{
		UE_LOGF(LogRigMapper, Warning, "Missing inputs field")
		SetDefinitionValid(false);
		return false;
	}
	for (const auto& InputAttr : JsonObject->GetArrayField(TEXT("inputs")))
	{
		const FString InputString = InputAttr->AsString();
		if (Inputs.Contains(InputString))
		{
			UE_LOGF(LogRigMapper, Warning, "Duplicate input was found: %ls", *InputString)
		}
		else
		{
			Inputs.Add(InputString);	
		}
	}
	if (Inputs.IsEmpty())
	{
		UE_LOGF(LogRigMapper, Warning, "Not enough inputs")
		SetDefinitionValid(false);
	}
	return bValidated;
}

bool URigMapperDefinition::LoadNullOutputsFromJsonObject(const TSharedPtr<FJsonObject>& JsonObject)
{
	if (JsonObject->HasTypedField(TEXT("null_outputs"), EJson::Array))
	{
		for (const auto& NullOutputAttr : JsonObject->GetArrayField(TEXT("null_outputs")))
		{
			const FString NullOutputString = NullOutputAttr->AsString();
			if (NullOutputs.Contains(NullOutputString))
			{
				UE_LOGF(LogRigMapper, Warning, "Duplicate null output was found: %ls", *NullOutputString)
			}
			else
			{
				if (Outputs.Contains(NullOutputString))
				{
					UE_LOGF(LogRigMapper, Warning, "Null output conflicts with existing output: %ls", *NullOutputString)
					SetDefinitionValid(false);
				}
				else
				{
					NullOutputs.Add(NullOutputString);	
				}
			}
		}
	}
	return WasDefinitionValidated();
}

void URigMapperDefinition::SetDefinitionValid(bool bValid)
{
	if (bValidated != bValid)
	{
		Modify();
		bValidated = bValid;
		UE_LOGF(LogRigMapper, Log, "Definition %ls is now %ls", *GetName(), WasDefinitionValidated() ? TEXT("valid") : TEXT("invalid"))
	}
}

bool URigMapperDefinition::LoadFeaturesFromJsonObject(const TSharedPtr<FJsonObject>& JsonObject, TArray<FString>& FeatureNames)
{
	if (!JsonObject->HasTypedField(TEXT("features"), EJson::Object))
	{
		UE_LOGF(LogRigMapper, Warning, "Missing features field")
		SetDefinitionValid(false);
		return false;
	}
	for (const auto& FeatureInfo : JsonObject->GetObjectField(TEXT("features"))->Values)
	{
		if (Inputs.Contains(FStringView(FeatureInfo.Key)))
		{
			UE_LOGF(LogRigMapper, Warning, "Feature conflicting with input of similar name: %ls", *FeatureInfo.Key)
			SetDefinitionValid(false);
			continue;
		}

		if (!Features.AddFromJsonObject(FString(FeatureInfo.Key), FeatureInfo.Value->AsObject()))
		{
			SetDefinitionValid(false);
			continue;
		}

		FeatureNames.Add(FString(FeatureInfo.Key));
	}
	return WasDefinitionValidated();
}

bool URigMapperDefinition::LoadOutputsFromJsonObject(const TSharedPtr<FJsonObject>& JsonObject, const TArray<FString>& FeatureNames)
{
	if (!JsonObject->HasTypedField(TEXT("outputs"), EJson::Object))
	{
		UE_LOGF(LogRigMapper, Warning, "Missing outputs field")
		SetDefinitionValid(false);
		return false;
	}
	for (const auto& OutputAttr : JsonObject->GetObjectField(TEXT("outputs"))->Values)
	{
		if (OutputAttr.Key.IsEmpty())
		{
			UE_LOGF(LogRigMapper, Warning, "Invalid output with empty name")
			continue;
		}
		FString OutputValue;
		if (!OutputAttr.Value.IsValid() || OutputAttr.Value->IsNull() || OutputAttr.Value->AsString().IsEmpty())
		{
			UE_LOGF(LogRigMapper, Warning, "Invalid value for output %ls", *OutputAttr.Key)
			SetDefinitionValid(false);
		}
		else
		{
			OutputValue = OutputAttr.Value->AsString();
			if (!Inputs.Contains(OutputValue) && !FeatureNames.Contains(OutputValue))
			{
				UE_LOGF(LogRigMapper, Warning, "Could not find corresponding input/feature for output %ls (%ls)", *OutputAttr.Key, *OutputValue)
				SetDefinitionValid(false);
			}
		}
		Outputs.Add(FString(OutputAttr.Key), OutputValue);
	}
	if (Outputs.IsEmpty())
	{
		UE_LOGF(LogRigMapper, Warning, "Not enough outputs")
		SetDefinitionValid(false);
	}
	return WasDefinitionValidated();
}

#if WITH_EDITOR
EDataValidationResult URigMapperDefinition::IsDataValid(FDataValidationContext& Context) const
{
	return IsDefinitionValid() ? EDataValidationResult::Valid : EDataValidationResult::Invalid;
}
#endif

bool URigMapperDefinition::ExportAsJsonString(FString& OutJsonString) const
{
	if (!IsDefinitionValid())
	{
		return false;
	}
	
	const TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>::Create(&OutJsonString);

	JsonWriter->WriteObjectStart();
	JsonWriter->WriteValue(TEXT("inputs"), Inputs);
	JsonWriter->WriteObjectStart(TEXT("features"));
	for (const FRigMapperMultiplyFeature& Feature: Features.Multiply)
	{
		JsonWriter->WriteObjectStart(Feature.Name);
		JsonWriter->WriteValue(TEXT("type"), TEXT("multiply"));
		JsonWriter->WriteValue(TEXT("input_features"), Feature.Inputs);
		JsonWriter->WriteArrayStart(TEXT("input_params"));
		JsonWriter->WriteArrayEnd();
		JsonWriter->WriteObjectStart(TEXT("params"));
		JsonWriter->WriteObjectEnd();
		JsonWriter->WriteObjectEnd();
	}
	for (const FRigMapperWsFeature& Feature: Features.WeightedSums)
	{
		JsonWriter->WriteObjectStart(Feature.Name);
		JsonWriter->WriteValue(TEXT("type"), TEXT("weighted_sum"));
		TArray<FString> InputNames;
		TArray<double> InputWeights;
		Feature.Inputs.GetKeys(InputNames);
		InputWeights.Reserve(InputNames.Num());
		for (const FString& Input : InputNames)
		{
			InputWeights.Add(Feature.Inputs[Input]);
		}
		JsonWriter->WriteValue(TEXT("input_features"), InputNames);
		JsonWriter->WriteArrayStart(TEXT("input_params"));
		JsonWriter->WriteArrayEnd();
		JsonWriter->WriteObjectStart(TEXT("params"));
		JsonWriter->WriteValue(TEXT("weights"), InputWeights);
		if (Feature.Range.bHasLowerBound)
		{
			JsonWriter->WriteValue(TEXT("min"), Feature.Range.LowerBound);
		}
		if (Feature.Range.bHasUpperBound)
		{
			JsonWriter->WriteValue(TEXT("max"), Feature.Range.UpperBound);
		}
		JsonWriter->WriteObjectEnd();
		JsonWriter->WriteObjectEnd();
	}
	for (const FRigMapperSdkFeature& Feature: Features.SDKs)
	{
		JsonWriter->WriteObjectStart(Feature.Name);
		JsonWriter->WriteValue(TEXT("type"), TEXT("sdk"));
		JsonWriter->WriteValue(TEXT("input_features"), TArray<FString>({Feature.Input}));
		JsonWriter->WriteArrayStart(TEXT("input_params"));
		JsonWriter->WriteArrayEnd();
		JsonWriter->WriteObjectStart(TEXT("params"));
		TArray<double> Keys;
		TArray<double> Values;
		Keys.Reserve(Feature.Keys.Num());
		Values.Reserve(Feature.Keys.Num());
		for (const FRigMapperSdkKey& Key : Feature.Keys)
		{
			Keys.Add(Key.In);
			Values.Add(Key.Out);
		}
		JsonWriter->WriteValue(TEXT("in_val"), Keys);
		JsonWriter->WriteValue(TEXT("out_val"), Values);	
		JsonWriter->WriteObjectEnd();
		JsonWriter->WriteObjectEnd();
	}
	for (const FRigMapperMathFeature& Feature : Features.MathOps)
	{
		JsonWriter->WriteObjectStart(Feature.Name);
		JsonWriter->WriteValue(TEXT("type"), TEXT("math_op"));
		JsonWriter->WriteValue(TEXT("operation"), MathOpEnumToString[Feature.Operation]);
		int32 InputCount = Feature.Inputs.Num();
		TArray<FString> InputNames;
		InputNames.Reserve(InputCount);
		TArray<double> InputConstants;
		InputConstants.Reserve(InputCount);
		for (const FRigMapperMathInput& Input : Feature.Inputs)
		{
			InputNames.Add(Input.NodeName);
			InputConstants.Add(Input.ConstantValue);
		}
		JsonWriter->WriteValue(TEXT("input_features"), InputNames);
		JsonWriter->WriteArrayStart(TEXT("input_params"));
		JsonWriter->WriteArrayEnd();
		JsonWriter->WriteObjectStart(TEXT("params"));
		JsonWriter->WriteValue(TEXT("constants"), InputConstants);
		JsonWriter->WriteObjectEnd();
		JsonWriter->WriteObjectEnd();
	}
	JsonWriter->WriteObjectEnd();
	JsonWriter->WriteObjectStart(TEXT("parameters"));
	JsonWriter->WriteObjectEnd();
	JsonWriter->WriteObjectStart(TEXT("outputs"));
	for (const TPair<FString, FString>& Output : Outputs)
	{
		JsonWriter->WriteValue(Output.Key, Output.Value);	
	}
	JsonWriter->WriteObjectEnd();
	JsonWriter->WriteValue(TEXT("null_outputs"), NullOutputs);
	JsonWriter->WriteObjectEnd();
	JsonWriter->Close();
	return true;
}

bool URigMapperDefinition::ExportAsJsonFile(const FFilePath& JsonFilePath) const
{
	FString JsonString;
	
	if (ExportAsJsonString(JsonString))
	{
		return FFileHelper::SaveStringToFile(JsonString, *JsonFilePath.FilePath);
	}
	return false;
}

bool URigMapperDefinition::LoadInputsFromSkeletalMesh(const TArray<FName>& InNewInputCurves)
{
	bool bInputAdded = false;
	if (!InNewInputCurves.IsEmpty())
	{
		TSet<FName> ExistingCurves;
		for (const FString& Input : Inputs)
		{
			ExistingCurves.Add(FName(Input));
		}

		for (const FName& NewInput : InNewInputCurves)
		{
			if (!ExistingCurves.Contains(NewInput))
			{
				Inputs.Add(NewInput.ToString());
				ExistingCurves.Add(NewInput);  // keep set in sync
				bInputAdded = true;
			}
		}
	}
	if (bInputAdded)
	{
		FRigMapperDefinitionsSingleton::Get().ClearFromCache(this);
		OnRigMapperDefinitionCreated.Broadcast();
		SetDefinitionValid(false);
	}
	return bInputAdded;
}

bool URigMapperDefinition::LoadOutputsFromSkeletalMesh(const TArray<FName>& InNewOutputCurves)
{
	bool bOutputAdded = false;
	if (!InNewOutputCurves.IsEmpty())
	{
		for (const FName& NewOutput : InNewOutputCurves)
		{
			if (!Outputs.Contains(NewOutput.ToString()) && !NullOutputs.Contains(NewOutput.ToString()))
			{
				// If new output is not already used, import it, otherwise leave it as is, even if it was a NullOutput.
				Outputs.Add(NewOutput.ToString(), TEXT(""));				
				bOutputAdded = true;
			}
		}
	}
	if (bOutputAdded)
	{
		FRigMapperDefinitionsSingleton::Get().ClearFromCache(this);
		OnRigMapperDefinitionCreated.Broadcast();
		SetDefinitionValid(false);
	}
	return bOutputAdded;
}

bool URigMapperDefinition::WasDefinitionValidated() const
{
	return bValidated;
}

bool URigMapperDefinition::IsDefinitionValid(bool bWarn, bool bForce) const
{
	return IsDefinitionValid(nullptr, bWarn, bForce);
}

bool URigMapperDefinition::IsDefinitionValid(FRigMapperValidationContext* OutContext, bool bWarn, bool bForce) const
{
	if (!bForce && WasDefinitionValidated())
	{
		return true;
	}

	bool bValid = true;

	if (Inputs.IsEmpty())
	{
		const FString Message = TEXT("Not enough inputs");
		if (OutContext)
		{
			OutContext->AddError(Message);
		}
		else
		{
			UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
		}
		bValid = false;
	}
	if (Outputs.IsEmpty())
	{
		const FString Message = TEXT("Not enough outputs");
		if (OutContext)
		{
			OutContext->AddError(Message);
		}
		else
		{
			UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
		}
		bValid = false;
	}

	TArray<FString> CheckInputs;
	CheckInputs.Reserve(Inputs.Num());
	for (const FString& Input : Inputs)
	{
		// todo: check with skeleton and or control rig ?
		if (CheckInputs.Contains(Input))
		{
			const FString Message = FString::Printf(TEXT("Duplicate input %ls"), *Input);
			if (OutContext)
			{
				OutContext->AddError(Message);
			}
			else
			{
				UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
			}
			bValid = false;
		}
		else
		{
			CheckInputs.Add(Input);
		}
	}
	TArray<FString> FeatureNames;
	Features.GetFeatureNames(FeatureNames);
	for (const TPair<FString, FString>& Output : Outputs)
	{
		// todo: check with skeleton and or control rig ?
		if (!Inputs.Contains(Output.Value) && !FeatureNames.Contains(Output.Value))
		{
			const FString Message = FString::Printf(TEXT("Output %ls does not link to any existing input or feature"), *Output.Key);
			if (OutContext)
			{
				OutContext->AddError(Message);
			}
			else
			{
				UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
			}
			bValid = false;
		}
	}
	
	bValid &= Features.IsValid(Inputs, bWarn, OutContext);

	// check that NullOutputs does not contain any keys from Outputs, or any duplicated keys
	TArray<FString> CheckNullOutputs;
	CheckNullOutputs.Reserve(NullOutputs.Num());

	for (const FString& NullOutput : NullOutputs)
	{
		if (Outputs.Contains(NullOutput))
		{
			const FString Message = FString::Printf(TEXT("Output is also defined as a NullOutput %ls"), *NullOutput);
			if (OutContext)
			{
				OutContext->AddError(Message);
			}
			else
			{
				UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
			}
			bValid = false;
		}
		if (CheckNullOutputs.Contains(NullOutput))
		{
			const FString Message = FString::Printf(TEXT("Duplicate NullOutput %ls"), *NullOutput);
			if (OutContext)
			{
				OutContext->AddError(Message);
			}
			else
			{
				UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
			}
			bValid = false;
		}
		else
		{
			CheckNullOutputs.Add(NullOutput);
		}
	}

	return bValid;
}

bool URigMapperDefinition::Validate()
{
	return Validate(nullptr);
}

bool URigMapperDefinition::Validate(FRigMapperValidationContext* OutContext)
{
	UE_LOGF(LogRigMapper, Log, "Validating definition %ls", *GetName())

	const bool bPrev = WasDefinitionValidated();
	
	SetDefinitionValid(IsDefinitionValid(OutContext, true, true));

	if (WasDefinitionValidated() == bPrev)
	{
		UE_LOGF(LogRigMapper, Log, "Definition %ls is still %ls", *GetName(), WasDefinitionValidated() ? TEXT("valid") : TEXT("invalid"))
	}
	
	return WasDefinitionValidated();
}

void URigMapperDefinition::LogAll(const FRigMapperValidationContext& InContext) const
{
	InContext.LogAll();
}

#if WITH_EDITOR
void URigMapperDefinition::InvalidateCache()
{
	FRigMapperDefinitionsSingleton::Get().ClearFromCache(this);
}

void URigMapperDefinition::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// if the definition exists in the global cache, delete it so that the editor doesn't get out of sync with the cache
	FRigMapperDefinitionsSingleton::Get().ClearFromCache(this);

	OnRigMapperDefinitionUpdated.Broadcast();

	SetDefinitionValid(false);
}

void URigMapperDefinition::BroadcastDefinitionModified()
{
	// if the definition exists in the global cache, delete it so that the editor doesn't get out of sync with the cache
	FRigMapperDefinitionsSingleton::Get().ClearFromCache(this);

	OnRigMapperDefinitionUpdated.Broadcast();

	SetDefinitionValid(false);
}
#endif

bool URigMapperLinkedDefinitions::GetBakedInputRec(const FString& InputName, const int32 DefinitionIndex, FRigMapperFeature::FBakedInput& OutBakedInput, bool& bOutMissingIsNotNullOutput, FRigMapperValidationContext* OutContext)
{
	check(SourceDefinitions[DefinitionIndex])
	
	if (SourceDefinitions[DefinitionIndex]->Inputs.Contains(InputName))
	{
		if (DefinitionIndex > 0)
		{
			if (!SourceDefinitions[DefinitionIndex - 1]->Outputs.Contains(InputName))
			{
				if (!SourceDefinitions[DefinitionIndex - 1]->NullOutputs.Contains(InputName))
				{
					const FString Message = FString::Printf(TEXT("Input %ls on definition %ls (layer %d) does not match any output from definition %ls (layer %d)"),
						*InputName, *SourceDefinitions[DefinitionIndex]->GetName(), DefinitionIndex, *SourceDefinitions[DefinitionIndex - 1]->GetName(), DefinitionIndex - 1);
					if (OutContext)
					{
						OutContext->AddWarning(Message, DefinitionIndex);
					}
					else
					{
						UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
					}

					// Pass-through: trace back to the earliest definition that has this as an input
					// If it's a top-level input (layer 0), create a direct input reference
					for (int32 SearchIndex = DefinitionIndex - 1; SearchIndex >= 0; --SearchIndex)
					{
						if (SourceDefinitions[SearchIndex]->Outputs.Contains(InputName))
						{
							return GetBakedInputRec(SourceDefinitions[SearchIndex]->Outputs[InputName], SearchIndex, OutBakedInput, bOutMissingIsNotNullOutput, OutContext);
						}
						if (SourceDefinitions[SearchIndex]->Inputs.Contains(InputName))
						{
							if (SearchIndex == 0)
							{
								// Reached the first definition that is a direct pass-through input
								OutBakedInput.Key = MakeShared<FRigMapperFeature>(InputName);
								OutBakedInput.Value.Empty();
								return OutBakedInput.Key.IsValid();
							}
						}
						// Continue searching backwards
					}
				}
				// Pass-through either failed or input is a NullOutput in previous layer and is silently dropped
				return false;
			}
			return GetBakedInputRec(SourceDefinitions[DefinitionIndex - 1]->Outputs[InputName], DefinitionIndex - 1, OutBakedInput, bOutMissingIsNotNullOutput, OutContext);
		}
		
		OutBakedInput.Key = MakeShared<FRigMapperFeature>(InputName);
		OutBakedInput.Value.Empty();
		return OutBakedInput.Key.IsValid();
	}

	ERigMapperFeatureType FeatureType;
	if (FRigMapperFeature* Feature = SourceDefinitions[DefinitionIndex]->Features.Find(InputName, FeatureType))
	{
		return Feature->BakeInput(this, DefinitionIndex, OutBakedInput, OutContext);
	}
	const FString Message = FString::Printf(TEXT("Could not bake input %ls"), *InputName);
	if (OutContext)
	{
		OutContext->AddError(Message);
	}
	else
	{
		UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
	}
	return false;
}

void URigMapperLinkedDefinitions::LogAll(const FRigMapperValidationContext& InContext) const
{
	InContext.LogAll();
}

bool FRigMapperMultiplyFeature::BakeInput(URigMapperLinkedDefinitions* LinkedDefinitions, int32 DefinitionIndex, FBakedInput& OutBakedInput, FRigMapperValidationContext* OutContext) const
{
	// todo: refactor all bake input using base class
	const FString BakedInputName = FString::Printf(TEXT("%s:%d"), *Name, DefinitionIndex);
	
	const TSharedPtr<FRigMapperMultiplyFeature> BakedMultFeature = MakeShared<FRigMapperMultiplyFeature>(BakedInputName);
	OutBakedInput.Key = BakedMultFeature;
	OutBakedInput.Value.Empty();
	bool bMissingIsNotNullOutput = false;
	
	for (const FString& FeatureInput : Inputs)
	{
		FBakedInput SubFeatureBakedInput;
		if (!LinkedDefinitions->GetBakedInputRec(FeatureInput, DefinitionIndex, SubFeatureBakedInput, bMissingIsNotNullOutput, OutContext))
		{
			continue;
		}

		if (SubFeatureBakedInput.Key->GetFeatureType() == ERigMapperFeatureType::Multiply)
		{
			const FRigMapperMultiplyFeature* SubFeatureMultBakedInput = static_cast<FRigMapperMultiplyFeature*>(SubFeatureBakedInput.Key.Get());
			BakedMultFeature->Inputs.Append(SubFeatureMultBakedInput->Inputs);
		}
		else
		{
			BakedMultFeature->Inputs.Add(SubFeatureBakedInput.Key->Name);
			OutBakedInput.Value.Add(SubFeatureBakedInput.Key);
		}
		OutBakedInput.Value.Append(SubFeatureBakedInput.Value);
	}

	return OutBakedInput.Key.IsValid() && !bMissingIsNotNullOutput; // if we have a missing definition which is not defined as a NullOutput, flag a failure
}

bool FRigMapperMultiplyFeature::ContainsInput(const FString& InInput) const
{
	return Inputs.Contains(InInput);
}

bool FRigMapperWsFeature::BakeInput(URigMapperLinkedDefinitions* LinkedDefinitions, int32 DefinitionIndex, FBakedInput& OutBakedInput, FRigMapperValidationContext* OutContext) const
{
	const FString BakedInputName = FString::Printf(TEXT("%s:%d"), *Name, DefinitionIndex);
	
	const TSharedPtr<FRigMapperWsFeature> BakedWsFeature = MakeShared<FRigMapperWsFeature>(BakedInputName);
	OutBakedInput.Key = BakedWsFeature;
	bool bMissingIsNotNullOutput = false;
			
	for (const TPair<FString, double>& FeatureInput : Inputs)
	{
		FBakedInput SubFeatureBakedInput;
		if (!LinkedDefinitions->GetBakedInputRec(FeatureInput.Key, DefinitionIndex, SubFeatureBakedInput, bMissingIsNotNullOutput, OutContext))
		{
			continue;
		}

		if (SubFeatureBakedInput.Key->GetFeatureType() == ERigMapperFeatureType::WeightedSum)
		{
			const FRigMapperWsFeature* SubFeatureWsBakedInput = static_cast<FRigMapperWsFeature*>(SubFeatureBakedInput.Key.Get());

			for (const TPair<FString, double>& SubFeatureSubInput : SubFeatureWsBakedInput->Inputs)
			{
				// check for the existence of the SubFeatureSubInput.Key in the map already as we can get 'diamond' structures
				if (BakedWsFeature->Inputs.Contains(SubFeatureSubInput.Key))
				{
					BakedWsFeature->Inputs[SubFeatureSubInput.Key] += SubFeatureSubInput.Value * FeatureInput.Value;
				}
				else
				{
					BakedWsFeature->Inputs.Add(SubFeatureSubInput.Key, SubFeatureSubInput.Value * FeatureInput.Value);
				}
			}
		}
		else
		{
			BakedWsFeature->Inputs.Add(SubFeatureBakedInput.Key->Name, FeatureInput.Value);
			OutBakedInput.Value.Add(SubFeatureBakedInput.Key);
		}
		OutBakedInput.Value.Append(SubFeatureBakedInput.Value);
	}

	return OutBakedInput.Key.IsValid() && !bMissingIsNotNullOutput; // if we have a missing definition which is not defined as a NullOutput, flag a failure
}

bool FRigMapperWsFeature::ContainsInput(const FString& InInput) const
{
	return Inputs.Contains(InInput);
}

bool FRigMapperSdkFeature::BakeInput(URigMapperLinkedDefinitions* LinkedDefinitions, int32 DefinitionIndex, FBakedInput& OutBakedInput, FRigMapperValidationContext* OutContext) const
{
	const FString BakedInputName = FString::Printf(TEXT("%s:%d"), *Name, DefinitionIndex);
	
	const TSharedPtr<FRigMapperSdkFeature> BakedSdkFeature = MakeShared<FRigMapperSdkFeature>(BakedInputName);
	OutBakedInput.Key = BakedSdkFeature;

	FBakedInput SubFeatureBakedInput;
	bool bMissingIsNotNullOutput = false;

	if (!LinkedDefinitions->GetBakedInputRec(Input, DefinitionIndex, SubFeatureBakedInput, bMissingIsNotNullOutput, OutContext))
	{
		return false;
	}

	if (SubFeatureBakedInput.Key->GetFeatureType() == ERigMapperFeatureType::SDK)
	{
		const FRigMapperSdkFeature* SubFeatureSdkBakedInput = static_cast<FRigMapperSdkFeature*>(SubFeatureBakedInput.Key.Get());
		BakedSdkFeature->Input = SubFeatureSdkBakedInput->Input;
		BakeKeys(*SubFeatureSdkBakedInput, *this, BakedSdkFeature->Keys);
	}
	else
	{
		BakedSdkFeature->Input = SubFeatureBakedInput.Key->Name;
		BakedSdkFeature->Keys = Keys;
		OutBakedInput.Value.Add(SubFeatureBakedInput.Key);
	}
	OutBakedInput.Value.Append(SubFeatureBakedInput.Value);

	return OutBakedInput.Key.IsValid() && !bMissingIsNotNullOutput; // if we have a missing definition which is not defined as a NullOutput, flag a failure
}

bool FRigMapperSdkFeature::BakeKeys(const FRigMapperSdkFeature& InSdk, const FRigMapperSdkFeature& OutSdk, TArray<FRigMapperSdkKey>& BakedKeys)
{
	// Create an array to backward evaluate InSdk (First Layer)
	TArray<TPair<double,double>> InKeysForEval_Backward;
	InKeysForEval_Backward.Reserve(InSdk.Keys.Num());
	for (const FRigMapperSdkKey& Key : InSdk.Keys)
	{
		InKeysForEval_Backward.Add({ Key.Out, Key.In });
	}
	InKeysForEval_Backward.Sort([](const TPair<double,double>& A, const TPair<double,double>& B) { return A.Key < B.Key; } );

	// Create an array to evaluate OutSdk (Second Layer)
	TArray<TPair<double,double>> OutKeysForEval;
	OutKeysForEval.Reserve(OutSdk.Keys.Num());
	for (const FRigMapperSdkKey& Key : OutSdk.Keys)
	{
		OutKeysForEval.Add({ Key.In, Key.Out });
	}

	// Prepare to store our baked keys 
	BakedKeys.Reserve(FMath::Max(InSdk.Keys.Num(), OutSdk.Keys.Num()));

	// Bake all keys from InSdk (layer 1) using OutSdk
	for (const FRigMapperSdkKey& InKey : InSdk.Keys)
	{
		double Value = 0.f;

		if (!FacialRigMapping::FEvalNodePiecewiseLinear::Evaluate_Static(InKey.Out, OutKeysForEval, Value))
		{
			return false;
		}
		BakedKeys.Add({ InKey.In, Value });
	}

	// Because we might miss the precision from some keys of OutSdk (i.e. if we have 3 keys in OutSdk and 2 in InSdk), we now need to insert missing keys from OutSdk by reverse baking them using InSdk 
	for (const FRigMapperSdkKey& OutKey : OutSdk.Keys)
	{
		double ActualInValue = 0.f;
				
		if (!FacialRigMapping::FEvalNodePiecewiseLinear::Evaluate_Static(OutKey.In, InKeysForEval_Backward, ActualInValue))
		{
			return false;
		}
		
		for (int32 InKeyIndex = 0; InKeyIndex < InSdk.Keys.Num(); InKeyIndex++)
		{
			const bool bInsertBefore = ActualInValue < InSdk.Keys[InKeyIndex].In && (InKeyIndex == 0 || ActualInValue > InSdk.Keys[InKeyIndex - 1].In);
			const bool bInsertAfter = ActualInValue > InSdk.Keys[InKeyIndex].In && (InKeyIndex == InSdk.Keys.Num() - 1 || ActualInValue < InSdk.Keys[InKeyIndex + 1].In);
			
			if (bInsertBefore || bInsertAfter)
			{
				const int32 NewIndex = InKeyIndex + BakedKeys.Num() - InSdk.Keys.Num(); 
				BakedKeys.Insert({ ActualInValue, OutKey.Out }, bInsertBefore ? NewIndex : NewIndex + 1);
				break;
			}
		}
	}

	// Finally, strip any duplicate or incorrectly ordered In keys from the beginning and end.
	// todo: make sure the rig mapper allows duplicate Input keys in the middle (i.e -1;0;0;1 -> -100;-50;50;100 should be a valid use case)
	while (BakedKeys.Num() >= 2 && BakedKeys[0].In >= BakedKeys[1].In)
	{
		BakedKeys.RemoveAt(0);
	}
	while (BakedKeys.Num() >= 2 && BakedKeys[BakedKeys.Num() - 1].In <= BakedKeys[BakedKeys.Num() - 2].In)
	{
		BakedKeys.RemoveAt(BakedKeys.Num() - 1);
	}
	return true;
}

bool URigMapperLinkedDefinitions::AddBakedInputFeature(const TSharedPtr<FRigMapperFeature>& Feature) const
{
	if (Feature->GetFeatureType() == ERigMapperFeatureType::Multiply)
	{
		BakedDefinition->Features.Multiply.Add(*static_cast<FRigMapperMultiplyFeature*>(Feature.Get()));
	}
	else if (Feature->GetFeatureType() == ERigMapperFeatureType::WeightedSum)
	{
		BakedDefinition->Features.WeightedSums.Add(*static_cast<FRigMapperWsFeature*>(Feature.Get()));
	}
	else if (Feature->GetFeatureType() == ERigMapperFeatureType::SDK)
	{
		BakedDefinition->Features.SDKs.Add(*static_cast<FRigMapperSdkFeature*>(Feature.Get()));
	}
	else
	{
		if (!SourceDefinitions[0]->Inputs.Contains(Feature->Name))
		{
			UE_LOGF(LogRigMapper, Warning, "Baked input could not be found within the lower level inputs")
			return false;
		}
		BakedDefinition->Inputs.AddUnique(Feature->Name);
	}
	return true;
}

void URigMapperLinkedDefinitions::AddBakedInputs(const TArray<FRigMapperFeature::FBakedInput>& BakedInputs, const TArray<int32>& BakedOutputIndices, const TArray<TPair<FString, FString>>& PairedOutputs) const
{
	TArray<FString> FeatureNames;
	
	BakedDefinition->Empty();

	for (int32 InputIndex = 0; InputIndex < BakedInputs.Num(); ++InputIndex)
	{
		const int32 OutputIndex = BakedOutputIndices[InputIndex]; // BakedInputs and BakedOutputIndices are the same length so we're safe
		BakedDefinition->Outputs.Add(PairedOutputs[OutputIndex].Key, BakedInputs[InputIndex].Key->Name);

		if (!FeatureNames.Contains(BakedInputs[InputIndex].Key->Name) && AddBakedInputFeature(BakedInputs[InputIndex].Key))
		{
			FeatureNames.Add(BakedInputs[InputIndex].Key->Name);

			for (const TSharedPtr<FRigMapperFeature>& SubFeature : BakedInputs[InputIndex].Value)
			{
				if (!FeatureNames.Contains(SubFeature->Name) && !AddBakedInputFeature(SubFeature))
				{
					break;
				}
				FeatureNames.Add(SubFeature->Name);
			}
		}
	}
}

#if WITH_EDITOR
EDataValidationResult URigMapperLinkedDefinitions::IsDataValid(FDataValidationContext& Context) const
{
	return AreLinkedDefinitionsValid() ? EDataValidationResult::Valid : EDataValidationResult::Invalid;
}
#endif

bool URigMapperLinkedDefinitions::BakeDefinitions()
{
	return BakeDefinitions(nullptr);
}

bool URigMapperLinkedDefinitions::BakeDefinitions(FRigMapperValidationContext* OutContext)
{
	UE_LOGF(LogRigMapper, Log, "Baking linked definition %ls", *GetName())
	
	if (!BakedDefinition)
	{
		const FString Message = TEXT("Baked definition is unset");
		if (OutContext)
		{
			OutContext->AddError(Message);
		}
		else
		{
			UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
		}
		return false;
	}
	if (SourceDefinitions.Num() < 2 || !SourceDefinitions[0] || !SourceDefinitions.Last())
	{
		const FString Message = TEXT("Baking requires a minimum of 2 source definitions");
		if (OutContext)
		{
			OutContext->AddError(Message);
		}
		else
		{
			UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
		}
		return false;
	}
	for (int32 DefIndex = 0; DefIndex < SourceDefinitions.Num(); DefIndex++)
	{
		if (!SourceDefinitions[DefIndex]|| !SourceDefinitions[DefIndex]->IsDefinitionValid(OutContext, true, false))
		{
			const FString Message = FString::Printf(TEXT("Invalid source definition at index %d. Make sure to revalidate the asset if necessary"), DefIndex);
			if (OutContext)
			{
				OutContext->AddError(Message, DefIndex);
			}
			else
			{
				UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
			}
			return false;
		}
	}
	
	const TArray<TPair<FString, FString>>& PairedOutputs = SourceDefinitions.Last()->Outputs.Array();

	TArray<FRigMapperFeature::FBakedInput> BakedInputs;
	TArray<int32> BakedOutputIndices;
	BakedInputs.Reserve(PairedOutputs.Num());
	BakedOutputIndices.Reserve(PairedOutputs.Num());
	bool bMissingIsNotNullOutput = false;

	for (int32 Index = 0; Index < PairedOutputs.Num(); ++Index)
	{
		FRigMapperFeature::FBakedInput BakedInput;
		if (GetBakedInputRec(PairedOutputs[Index].Value, SourceDefinitions.Num() - 1, BakedInput, bMissingIsNotNullOutput, OutContext))
		{
			BakedInputs.Add(BakedInput);
			BakedOutputIndices.Add(Index);
		}
		else
		{
			const FString Message = FString::Printf(TEXT("Could not bake input %ls associated to output %ls. Output will be excluded."), *PairedOutputs[Index].Value, *PairedOutputs[Index].Key);
			if (OutContext)
			{
				OutContext->AddWarning(Message);
			}
			else
			{
				UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
			}
		}
	}

	AddBakedInputs(BakedInputs, BakedOutputIndices, PairedOutputs);

	UE_LOGF(LogRigMapper, Log, "Finished baking linked definition %ls", *GetName())
	
	return BakedDefinition->Validate(OutContext);
}

bool URigMapperLinkedDefinitions::Validate()
{
	return Validate(nullptr);
}

bool URigMapperLinkedDefinitions::Validate(FRigMapperValidationContext* OutContext)
{
	if (BakedDefinition)
	{
		BakedDefinition->Validate(OutContext);
	}

	for (URigMapperDefinition* Definition : SourceDefinitions)
	{
		if (Definition)
		{
			Definition->Validate(OutContext);
		}
	}

	return AreLinkedDefinitionsValid(OutContext);
}

bool URigMapperLinkedDefinitions::AreLinkedDefinitionsValid() const
{
	return AreLinkedDefinitionsValid(nullptr);
}

bool URigMapperLinkedDefinitions::AreLinkedDefinitionsValid(FRigMapperValidationContext* OutContext) const
{
	bool bOk = IsValid(BakedDefinition) && BakedDefinition->IsDefinitionValid(OutContext, false, false);
	
	if (!bOk)
	{
		UE_LOGF(LogRigMapper, Warning, "Failed to validate the baked definition")
	}

	for (int32 DefinitionIndex = 0; DefinitionIndex < SourceDefinitions.Num(); DefinitionIndex++)
	{
		if (!SourceDefinitions[DefinitionIndex])
		{
			const FString Message = FString::Printf(TEXT("Source definition %d is unset"), DefinitionIndex);
			if (OutContext)
			{
				OutContext->AddError(Message, DefinitionIndex);
			}
			else
			{
				UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
			}
			bOk = false;
			continue;
		}
		if (!SourceDefinitions[DefinitionIndex]->IsDefinitionValid(OutContext, false, false))
		{
			const FString Message = FString::Printf(TEXT("Source definition %d is invalid"), DefinitionIndex);
			if (OutContext)
			{
				OutContext->AddError(Message, DefinitionIndex);
			}
			else
			{
				UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
			}
			bOk = false;
		}
		if (DefinitionIndex > 0)
		{
			for (const FString& Input : SourceDefinitions[DefinitionIndex]->Inputs)
			{
				if (!SourceDefinitions[DefinitionIndex - 1]->Outputs.Contains(Input) && !SourceDefinitions[DefinitionIndex - 1]->NullOutputs.Contains(Input))
				{
					const FString Message = FString::Printf(TEXT("Could not find matching output in definition %d for input %ls in definition %d. Input will receive no value."), DefinitionIndex - 1, *Input, DefinitionIndex);
					if (OutContext)
					{
						OutContext->AddWarning(Message, DefinitionIndex - 1);
					}
					else
					{
						UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
					}
				}
			}
			for (const TPair<FString, FString>& Output : SourceDefinitions[DefinitionIndex - 1]->Outputs)
			{
				if (!SourceDefinitions[DefinitionIndex]->Inputs.Contains(Output.Key))
				{
					const FString Message = FString::Printf(TEXT("Could not find matching input in definition %d for output %ls in definition %d. Output will be dropped."), DefinitionIndex, *Output.Key, DefinitionIndex - 1);
					if (OutContext)
					{
						OutContext->AddWarning(Message, DefinitionIndex);
					}
					else
					{
						UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
					}
				}
			}

			for (const FString& Input : SourceDefinitions[DefinitionIndex]->Inputs)
			{
				// Check if this input is satisfied by a NullOutput from the previous layer
				if (SourceDefinitions[DefinitionIndex - 1]->NullOutputs.Contains(Input))
				{
					// Verify it's not used as a real input to any feature
					TArray<FString> FeatureNames;
					SourceDefinitions[DefinitionIndex]->Features.GetFeatureNames(FeatureNames);

					bool bUsedInFeature = false;
					for (const FString& FeatureName : FeatureNames)
					{
						ERigMapperFeatureType FeatureType;
						if (const FRigMapperFeature* Feature = SourceDefinitions[DefinitionIndex]->Features.Find(FeatureName, FeatureType))
						{
							if (Feature->ContainsInput(Input))
							{
								bUsedInFeature = true;
								break;
							}
						}
					}

					// Also check if it's directly referenced by an Output mapping
					bool bUsedInOutput = false;
					for (const TPair<FString, FString>& Output : SourceDefinitions[DefinitionIndex]->Outputs)
					{
						if (Output.Value == Input)
						{
							bUsedInOutput = true;
							break;
						}
					}

					if (bUsedInFeature || bUsedInOutput)
					{
						const FString Message = FString::Printf(TEXT("Input %ls in definition %d is sourced from a NullOutput in definition %d but is used in features/outputs. It will have no value at runtime."), *Input, DefinitionIndex, DefinitionIndex - 1);
						if (OutContext)
						{
							OutContext->AddWarning(Message, DefinitionIndex);
						}
						else
						{
							UE_LOG(LogRigMapper, Warning, TEXT("%s"), *Message)
						}
					}
				}
			}
		}
	}
	
	return bOk;
}

// Flush all messages to UE_LOG
inline void FRigMapperValidationContext::LogAll() const
{
	for (const FRigMapperValidationMessage& Msg : Messages)
	{
		switch (Msg.Severity)
		{
		case ERigMapperValidationSeverity::Error:
			UE_LOGF(LogRigMapper, Error, "%ls", *Msg.Message);
			break;
		case ERigMapperValidationSeverity::Warning:
			UE_LOGF(LogRigMapper, Warning, "%ls", *Msg.Message);
			break;
		}
	}
}
