// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigMapperLog.h"

DEFINE_LOG_CATEGORY(LogRigMapper);