// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigMapperModule.h"

#include "Modules/ModuleManager.h"

void FRigMapperModule::StartupModule()
{
}

void FRigMapperModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FRigMapperModule, RigMapper)
