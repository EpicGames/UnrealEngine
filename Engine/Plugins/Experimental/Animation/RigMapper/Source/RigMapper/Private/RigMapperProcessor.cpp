// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigMapperProcessor.h"
#include "RigMapperLog.h"
#include "RigMapperDefinition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigMapperProcessor)

DEFINE_STAT(STAT_RigMapperGetRigMapper);
DEFINE_STAT(STAT_RigMapperLoadDefinition);


FRigMapperDefinitionsSingleton& FRigMapperDefinitionsSingleton::Get()
{
	static FRigMapperDefinitionsSingleton SingletonInstance;
	return SingletonInstance;
}

void FRigMapperDefinitionsSingleton::ClearFromCache(const TObjectPtr<const URigMapperDefinition>& InDefinition)
{
	// remove the rig mapper from the cache if it exists
	RigMappers.Remove(InDefinition);
}

bool FRigMapperDefinitionsSingleton::GetRigMapper(const TObjectPtr<const URigMapperDefinition>& InDefinition, FacialRigMapping::FRigMapper& OutRigMapper)
{
	SCOPE_CYCLE_COUNTER(STAT_RigMapperGetRigMapper);
	OutRigMapper.Reset();
	FSoftObjectPath DefinitionSoftObjectPath(InDefinition);
	if (const FacialRigMapping::FRigMapper* RigMapper = RigMappers.Find(DefinitionSoftObjectPath); RigMapper != nullptr)
	{
		OutRigMapper = *RigMapper;
	}
	else
	{
		SCOPE_CYCLE_COUNTER(STAT_RigMapperLoadDefinition);
		if (!OutRigMapper.LoadDefinition(InDefinition))
		{
			UE_LOGF(LogRigMapper, Error, "Could not load definition (%ls)", *InDefinition->GetPathName())
			return false;
		}
		if (!InDefinition->WasDefinitionValidated())
		{
			UE_LOGF(LogRigMapper, Warning, "RigMapper definition asset (%ls) was not validated please ensure that all RigMapper node definitions have been set to validated assets.", *InDefinition->GetPathName())
		}
		RigMappers.Add(DefinitionSoftObjectPath, OutRigMapper);
	}
	return true;
}

FRigMapperProcessor::FRigMapperProcessor(const TArray<URigMapperDefinition*>& InDefinitions)
{
	for (const URigMapperDefinition* Definition : InDefinitions)
	{
		if (Definition)
		{
			FacialRigMapping::FRigMapper RigMapper;
			if (!FRigMapperDefinitionsSingleton::Get().GetRigMapper(Definition, RigMapper))

			{
				UE_LOGF(LogRigMapper, Error, "Could not load definition %ls", *Definition->GetPathName())
					RigMappers.Empty();
				return;
			}
			RigMappers.Add(MoveTemp(RigMapper));
		}
	}

	IndexCache.AddDefaulted(RigMappers.Num());
}

FRigMapperProcessor::FRigMapperProcessor(URigMapperDefinition* InDefinition)
{
	if (InDefinition)
	{
		FacialRigMapping::FRigMapper RigMapper;
		if (!FRigMapperDefinitionsSingleton::Get().GetRigMapper(InDefinition, RigMapper))

		{
			UE_LOGF(LogRigMapper, Error, "Could not load definition %ls", *InDefinition->GetPathName())
				RigMappers.Empty();
			return;
		}
		RigMappers.Add(MoveTemp(RigMapper));
		IndexCache.AddDefaulted(RigMappers.Num());
	}
}

bool FRigMapperProcessor::IsValid() const
{
	if (RigMappers.IsEmpty())
	{
		return false;
	}
	return true;
}

const TArray<FName>& FRigMapperProcessor::GetInputNames() const
{
	return RigMappers[0].GetInputNames();
}

const TArray<FName>& FRigMapperProcessor::GetOutputNames() const
{
	return RigMappers.Last().GetOutputNames();
}

bool FRigMapperProcessor::EvaluateFrames(const TArray<FName>& CurveNames, const TArray<FPoseValues>& InFrameValues, TArray<FPoseValues>& OutFrameValues)
{
	if (!IsValid())
	{
		return false;
	}
	bool bValid = true;
	
	for (int32 RigIndex = 0; RigIndex < RigMappers.Num(); RigIndex++)
	{
		OutFrameValues.Reset(InFrameValues.Num());
		OutFrameValues.AddDefaulted(InFrameValues.Num());
	
		for (int32 FrameIndex = 0; FrameIndex < InFrameValues.Num(); FrameIndex++)
		{
			if (RigIndex == 0)
			{
				bValid &= EvaluateFrame_Internal(CurveNames, InFrameValues[FrameIndex], OutFrameValues[FrameIndex], RigIndex);
			}
			else
			{
				bValid &= EvaluateFrame_Internal(RigMappers[RigIndex - 1].GetOutputNames(), OutFrameValues[FrameIndex], OutFrameValues[FrameIndex], RigIndex);
			}
		}
	}
	return bValid;
}

bool FRigMapperProcessor::EvaluateFrames(const TArray<FName>& CurveNames, const TArray<FPoseValues>& InFrameValues, TArray<FName>& OutCurveNames,
	TArray<FPoseValues>& OutFrameValues)
{
	if (!IsValid() || !EvaluateFrames(CurveNames, InFrameValues, OutFrameValues))
	{
		return false;
	}
	const TArray<FName>& NewOutputNames = RigMappers.Last().GetOutputNames();
	OutCurveNames.Reset(NewOutputNames.Num());
	OutCurveNames.Append(NewOutputNames);
	return true;
}

bool FRigMapperProcessor::EvaluateFrames_Interp(const TArray<FName>& CurveNames, const TArray<FPoseValues>& InFrameValues, TArray<FPoseValues>& OutFrameValues, const TArray<FFrameTime>& FrameTimes)
{
	// todo
	if (!IsValid() || !EvaluateFrames(CurveNames, InFrameValues, OutFrameValues))
	{
		return false;
	}
	return true;
}

bool FRigMapperProcessor::EvaluateFrame(const TArray<FName>& CurveNames, const FPoseValues& InCurveValues, FPoseValues& OutCurveValues)
{
	if (!IsValid())
	{
		return false;
	}
	bool bValid = true;
	
	for (int32 RigIndex = 0; RigIndex < RigMappers.Num(); RigIndex++)
	{
		if (RigIndex == 0)
		{
			bValid &= EvaluateFrame_Internal(CurveNames, InCurveValues, OutCurveValues, RigIndex);
		}
		else
		{
			bValid &= EvaluateFrame_Internal(RigMappers[RigIndex - 1].GetOutputNames(), OutCurveValues, OutCurveValues, RigIndex);
		}
	}

	return true;
}

bool FRigMapperProcessor::EvaluateFrame(const TArray<FName>& CurveNames, const FPoseValues& InCurveValues, TArray<FName>& OutCurveNames,
	FPoseValues& OutCurveValues)
{
	if (!IsValid() || !EvaluateFrame(CurveNames, InCurveValues, OutCurveValues))
	{
		return false;
	}
	const TArray<FName>& NewOutputNames = RigMappers.Last().GetOutputNames();
	OutCurveNames.Reset(NewOutputNames.Num());
	OutCurveNames.Append(NewOutputNames);
	return true;
}


bool FRigMapperProcessor::EvaluateFrame_Internal(const TArray<FName>& CurveNames, const FPoseValues& InCurveValues, FPoseValues& OutCurveValues, const int32 RigIndex)
{
	if (!RigMappers.IsValidIndex(RigIndex) || !RigMappers[RigIndex].IsValid() || CurveNames.Num() != InCurveValues.Num())
	{
		return false;
	}
	if (IndexCache[RigIndex].Max() != CurveNames.Num())
	{
		IndexCache[RigIndex].Reset(CurveNames.Num());
	}
		
	RigMappers[RigIndex].SetDirty();

	// Set inputs
	const TArray<FName>& InputNames = RigMappers[RigIndex].GetInputNames();
		
	for (int32 CurveIndex = 0; CurveIndex < CurveNames.Num(); CurveIndex++)
	{
		// Cache input indices
		if (!IndexCache[RigIndex].IsValidIndex(CurveIndex))
		{
			IndexCache[RigIndex].Add(InputNames.Find(CurveNames[CurveIndex]));
		}

		// Set input from curve
		const int32 InputIndex = IndexCache[RigIndex][CurveIndex];
		if (InputNames.IsValidIndex(InputIndex) && InCurveValues[CurveIndex].IsSet())
		{
			RigMappers[RigIndex].SetDirectValue(InputIndex, InCurveValues[CurveIndex].GetValue());
		}
	}
	
	RigMappers[RigIndex].GetOptionalFloatOutputValuesInOrder(OutCurveValues);
	return true;
}
