// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigMapperUtils.h"
#include "RigMapperDefinition.h"
#include "Algo/Find.h"
#include "Algo/MaxElement.h"

const TMap<ERigMapperFeatureType, FString> FeatureTypePrefixMap = {
	{ ERigMapperFeatureType::WeightedSum, TEXT("ws:") },
	{ ERigMapperFeatureType::Multiply, TEXT("mul:") },
	{ ERigMapperFeatureType::MathOp, TEXT("math:") },
	{ ERigMapperFeatureType::SDK, TEXT("sdk:") },
	{ ERigMapperFeatureType::Input, TEXT("in:") },
	{ ERigMapperFeatureType::Output, TEXT("out:") },
	{ ERigMapperFeatureType::NullOutput, TEXT("null:") },
	{ ERigMapperFeatureType::Invalid, TEXT("invalid:") }
};

const FString GenericNodeName = TEXT("node");

FString FRigMapperUtils::GenerateUniqueFeatureName(const TArray<FString>& InExistingNames, FString InDesiredName, ERigMapperFeatureType InFeatureType)
{
	if (InFeatureType != ERigMapperFeatureType::Output && InFeatureType != ERigMapperFeatureType::NullOutput)
	{
		if (InFeatureType != ERigMapperFeatureType::Input)
		{
			InDesiredName = [&InDesiredName, InFeatureType]()
				{
					const FString& Prefix = FeatureTypePrefixMap[InFeatureType];
					if (InDesiredName.IsEmpty() || InDesiredName.StartsWith(Prefix))
					{
						return InDesiredName;
					}
					if ((InFeatureType != ERigMapperFeatureType::Multiply && InDesiredName.RemoveFromStart(FeatureTypePrefixMap[ERigMapperFeatureType::Multiply])) ||
						(InFeatureType != ERigMapperFeatureType::MathOp && InDesiredName.RemoveFromStart(FeatureTypePrefixMap[ERigMapperFeatureType::MathOp])) ||
						(InFeatureType != ERigMapperFeatureType::WeightedSum && InDesiredName.RemoveFromStart(FeatureTypePrefixMap[ERigMapperFeatureType::WeightedSum])) ||
						(InFeatureType != ERigMapperFeatureType::SDK && InDesiredName.RemoveFromStart(FeatureTypePrefixMap[ERigMapperFeatureType::SDK])))
					{
						return Prefix + InDesiredName;
					}
					return Prefix + InDesiredName;
				}();
		}
	}

	if (InDesiredName.IsEmpty())
	{
		InDesiredName = FeatureTypePrefixMap[InFeatureType] + GenericNodeName;
	}

	// If the desired name is unique, just use that.
	if (Algo::Find(InExistingNames, InDesiredName) == nullptr)
	{
		return InDesiredName;
	}

	// Create a unique name
	FName UniqueName = [&InDesiredName, InFeatureType]()
		{
			int32 Len = InDesiredName.Len();
			while (Len > 0 && FChar::IsDigit(InDesiredName[Len - 1]))
			{
				--Len;
			}		

			int32 Number;
			if (!LexTryParseString(Number, *InDesiredName.RightChop(Len)))
			{
				return FName(*InDesiredName.Left(Len));
			}

			if (Len > 0 && InDesiredName[Len - 1] == TCHAR('_'))
			{
				--Len;
			}

			return FName(*InDesiredName.Left(Len), ++Number);
		}();

	const int32 MinNumDigitsInPostfix = 1;
	const auto IsUniqueLambda = [MinNumDigitsInPostfix](TArray<FString> Others, const FName& InName)
		{
			const FString NameString = InName.ToString();

			const FString& PlainNameString = InName.GetPlainNameString();
			const int32 Number = NAME_INTERNAL_TO_EXTERNAL(InName.GetNumber());
			const FString PossibleNameString = FString::Printf(TEXT("%s_%0*d"), *PlainNameString, MinNumDigitsInPostfix, Number);

			for (const FString& Other : Others)
			{
				if (NameString == Other ||
					PossibleNameString == Other)
				{
					return false;
				}
			}

			return true;
		};

	// Only test names that start with the plain name string to optimize for performance
	TArray<FString> FilteredExistingNames;
	Algo::TransformIf(InExistingNames, FilteredExistingNames,
		[UniqueNameString = UniqueName.GetPlainNameString()](const FString& ExistingName)
		{
			return ExistingName.StartsWith(UniqueNameString);
		},
		[](const FString& ExistingName)
		{
			return ExistingName;
		});

	// Sort in descending order to optimize for performance
	Algo::Sort(FilteredExistingNames, [](const FString& NameA, const FString& NameB)
		{
			return NameA > NameB;
		});

	while (!IsUniqueLambda(FilteredExistingNames, UniqueName))
	{
		UniqueName.SetNumber(UniqueName.GetNumber() + 1);
	}

	const int32 UniqueNumber = NAME_INTERNAL_TO_EXTERNAL(UniqueName.GetNumber());
	if (UniqueNumber == 0)
	{
		return UniqueName.ToString();
	}
	else
	{
		return FString::Printf(TEXT("%s_%0*d"), *UniqueName.GetPlainNameString(), MinNumDigitsInPostfix, UniqueNumber);
	}
}
