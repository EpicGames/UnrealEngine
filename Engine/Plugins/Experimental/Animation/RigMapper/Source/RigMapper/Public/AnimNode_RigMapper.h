// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimNodeBase.h"

#include "CoreMinimal.h"
#include "RigMapperProcessor.h"

#include "AnimNode_RigMapper.generated.h"

#define UE_API RIGMAPPER_API

class URigMapperDefinitionUserData;
class URigMapperDefinition;
struct FRigControlElement;
struct FRigCurveElement;
class UControlRig;
class USkeletalMesh;

DECLARE_CYCLE_STAT_EXTERN(TEXT("Evaluate Rig Mapper Node"), STAT_RigMapperEvaluateNode, STATGROUP_RigMapper, RIGMAPPER_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Evaluate Rig Mapping"), STAT_RigMapperEvaluateRigs, STATGROUP_RigMapper, RIGMAPPER_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Initialize Rig Mapping"), STAT_RigMapperInitialize, STATGROUP_RigMapper, RIGMAPPER_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Rig Mapper Set Pose Curves"), STAT_RigMapperSetCurves, STATGROUP_RigMapper, RIGMAPPER_API);

using FRigMapperIndexMap = TArray<int32>;

/**
 * 
 */
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_RigMapper : public FAnimNode_Base
{
	GENERATED_USTRUCT_BODY()
	
public:
	UE_API FAnimNode_RigMapper();
	UE_API virtual ~FAnimNode_RigMapper() override;
	
	// FAnimNode_Base interface
	UE_API virtual void OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance) override;
	UE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	UE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	virtual bool HasPreUpdate() const override { return true; }
	UE_API virtual void PreUpdate(const UAnimInstance* InAnimInstance) override;
	UE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	UE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	UE_API virtual void Evaluate_AnyThread(FPoseContext & Output) override;
	UE_API virtual int32 GetLODThreshold() const override;
	// End of FAnimNode_Base interface

	// Initialize the rig mapper(s) from the current definitions or SKM asset user data and cache the curve indices & index mappings needed for evaluation
	UE_API bool InitializeRigMapping(USkeletalMesh* TargetMesh=nullptr);

	// Evaluate the new curve values using the initialized rig mappers and sets the new output pose 
	UE_API void EvaluateRigMapping(FPoseContext& Output);

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Links)
	FPoseLink SourcePose;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RigMapper, meta = (PinHiddenByDefault))
	TArray<TObjectPtr<URigMapperDefinition>> Definitions;
	
	/*
	* Max LOD that this node is allowed to run
	* For example if you have LODThreshold to be 2, it will run until LOD 2 (based on 0 index)
	* when the component LOD becomes 3, it will stop update/evaluate
	* currently transition would be issue and that has to be re-visited
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Performance, meta = (DisplayName = "LOD Threshold"))
	int32 LODThreshold = INDEX_NONE;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha, meta = (PinShownByDefault))
	float Alpha = 1.0f;
	
private:
	// The definitions that we have loaded. Cached to check against changes and reinit if need be
	TArray<TObjectPtr<URigMapperDefinition>> LoadedDefinitions;

	// The asset user data currently used to override definitions, if any was set on the skeletal mesh
	TObjectPtr<URigMapperDefinitionUserData> LoadedUserData;

	// The processor to evaluate the rig mapping
	FRigMapperProcessor RigMapperProcessor;
	
	// The cached input values passed to the rig mapper processor to avoid reallocations
	FRigMapperProcessor::FPoseValues CachedInputValues;
	
	// The cached output values passed to the rig mapper processor to avoid reallocations
	FRigMapperProcessor::FPoseValues CachedOutputValues;
	
	// Base curve mapping for bulk get/set of the linked pose curves
	struct FRigMapperCurveMapping
	{
		FRigMapperCurveMapping() = default;
		
		FRigMapperCurveMapping(FName InName, int32 InCurveIndex)
			: Name(InName)
			, CurveIndex(InCurveIndex)
		{}

		FName Name = NAME_None;
		int32 CurveIndex = INDEX_NONE;
	};

	// Curve mapping for bulk set of the linked pose curves with a mapping to the matching input to allow lerping with the current curve value
	struct FRigMapperOutputCurveMapping : FRigMapperCurveMapping
	{
		FRigMapperOutputCurveMapping() = default;
		
		FRigMapperOutputCurveMapping(FName InName, int32 InOutputCurveIndex, int32 InInputCurveIndex)
			: FRigMapperCurveMapping(InName, InOutputCurveIndex)
			, InputCurveIndex(InInputCurveIndex)
		{}
		
		int32 InputCurveIndex = INDEX_NONE;
	};

	// Input curve mapping for bulk get of the linked pose curve values
	using FInputCurveMappings = UE::Anim::TNamedValueArray<FDefaultAllocator, FRigMapperCurveMapping>;
	FInputCurveMappings InputCurveMappings;
	
	// Output curve mapping for the bulk get of the linked pose curve values
	using FOutputCurveMappings = UE::Anim::TNamedValueArray<FDefaultAllocator, FRigMapperOutputCurveMapping>;
	FOutputCurveMappings OutputCurveMappings;
};

#undef UE_API
