// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API RIGMAPPER_API

struct FRigMapperMultiplyFeature;
struct FRigMapperWsFeature;
struct FRigMapperSdkFeature;
struct FRigMapperMathFeature;
class URigMapperDefinition;
enum class ERigMapperMathOperation : uint8;

namespace FacialRigMapping
{
	class FNodePtr;

	class FInputNode;
	class FEvalNodeWeightedSum;
	class FEvalNodePiecewiseLinear;
	class FEvalNodeMultiply;
	class FEvalNodeMathOp;

	struct FNodeCollection
	{
		TArray<FInputNode> InputNodes;
		TArray<FEvalNodeWeightedSum> WeightedSumNodes;
		TArray<FEvalNodePiecewiseLinear> PiecewiseLinearNodes;
		TArray<FEvalNodeMultiply> MultiplyNodes;
		TArray<FEvalNodeMathOp> MathOpNodes;
	};

	enum class ENodeType : uint8
	{
		None,
		Input,
		WeightedSum,
		PiecewiseLinear,
		Multiply,
		MathOp
	};

	/**
	 * Lighweight pointer/proxy to a node in the FNodeCollection.
	 */
	class FNodePtr
	{
	public:
		FNodePtr() = default;
		FNodePtr(ENodeType InNodeType, int32 InDataIndex);

		bool TryGetValue(const FNodeCollection& NodeCollection, double& OutValue) const;
		double GetValue(const FNodeCollection& NodeCollection) const;
		void SetDirect(FNodeCollection& NodeCollection, double Value);
		void Reset(FNodeCollection& NodeCollection);

		bool IsValid() const
		{
			return NodeType != ENodeType::None;
		}

		bool IsInitialized(const FNodeCollection& NodeCollection) const;

		friend bool operator==(const FNodePtr& Lhs, const FNodePtr& Rhs)
		{
			return Lhs.DataIndex == Rhs.DataIndex && Lhs.NodeType == Rhs.NodeType;
		}

		friend uint32 GetTypeHash(const FNodePtr& Node)
		{
			return HashCombineFast(GetTypeHash(Node.DataIndex), GetTypeHash(Node.NodeType));
		}

	private:
		int32 DataIndex = INDEX_NONE;
		ENodeType NodeType = ENodeType::None;
	};

	/**
	 * Rig Mapper node representation loaded from URigMapperDefinition
	 */
	class FRigMapper
	{
	public:
		UE_API bool LoadDefinition(const URigMapperDefinition* Definition);
		UE_API bool SetDirectValue(int32 Index, double Value);
		UE_API bool SetDirectValue(const FName& InputName, double Value);
		UE_API void Reset();
		UE_API bool IsValid() const;

		UE_API TMap<FName,double> GetOutputValues(bool bSkipUnset = false) const;
		UE_API const TArray<FName>& GetOutputNames() const;
		UE_API void GetOutputValuesInOrder(TArray<double>& OutValues) const;
		UE_API void GetOptionalOutputValuesInOrder(TArray<TOptional<double>>& OutValues) const;
		UE_API void GetOptionalFloatOutputValuesInOrder(TArray<TOptional<float>>& OutValues) const;

		UE_API void SetDirty();

		UE_API const TArray<FName>& GetInputNames() const;

	private:
		TMap<FName, FNodePtr> Nodes;
		FNodeCollection NodeCollection;

		TArray<FNodePtr> OutputNodes;
		TArray<FName> OutputNames;

		TArray<FNodePtr> InputNodes;
		TArray<FName> InputNames;
	};


	class FInputNode
	{
	public:
		bool TryGetValue(const FNodeCollection& NodeCollection, double& OutValue) const
		{
			if (CachedValue.IsSet())
			{
				OutValue = CachedValue.GetValue();
				return true;
			}
			return false;
		}
		
		void SetDirect(double Value)
		{
			CachedValue = Value;
		}
		
		void Reset()
		{
			CachedValue.Reset();
		}

	protected:
		mutable TOptional<double> CachedValue;
	};
	
	template<typename DerivedNode>
	class TEvalNode
	{
	public:
		bool TryGetValue(const FNodeCollection& NodeCollection, double& OutValue) const
		{
			if (bInitialized)
			{
				if (!CachedValue.IsSet())
				{
					double Value;
					if (static_cast<const DerivedNode*>(this)->Evaluate(NodeCollection, Value))
					{
						CachedValue = Value;
						OutValue = Value;
						return true;
					}
				}
				else
				{
					OutValue = CachedValue.GetValue();
					return true;
				}
			}
			return false;
		}

		bool IsInitialized() const
		{
			return bInitialized;
		}

		void SetDirect(double Value)
		{
			CachedValue = Value;
		}

		void Reset()
		{
			CachedValue.Reset();
		}

	protected:
		mutable TOptional<double> CachedValue;
		bool bInitialized = false;
	};

	class FEvalNodeWeightedSum : public TEvalNode<FEvalNodeWeightedSum>
	{
	public:
		void Initialize(const FRigMapperWsFeature& FeatureDefinition, const TMap<FName, FNodePtr>& Nodes);
		bool Evaluate(const FNodeCollection& NodeCollection, double& OutValue) const;

	private:
		TArray<TPair<FNodePtr, double>> WeightedLinkedInputs;
		TRange<double> Range;
	};

	class FEvalNodePiecewiseLinear : public TEvalNode<FEvalNodePiecewiseLinear>
	{
	public:
		void Initialize(const FRigMapperSdkFeature& FeatureDefinition, const TMap<FName, FNodePtr>& Nodes);
		bool Evaluate(const FNodeCollection& NodeCollection, double& OutValue) const;

		static bool Evaluate_Static(const double InputValue, TConstArrayView<TPair<double, double>> Keys, double& OutValue);

	private:
		FNodePtr LinkedInput;
		// 95% only have 2 elements in this array.
		TArray<TPair<double, double>, TInlineAllocator<2>> Keys;
	};

	class FEvalNodeMultiply : public TEvalNode<FEvalNodeMultiply>
	{
	public:
		void Initialize(const FRigMapperMultiplyFeature& FeatureDefinition, const TMap<FName, FNodePtr>& Nodes);
		bool Evaluate(const FNodeCollection& NodeCollection, double& OutValue) const;

	private:
		TArray<FNodePtr, TInlineAllocator<2>> LinkedInputs;
	};

	class FEvalNodeMathOp : public TEvalNode<FEvalNodeMathOp>
	{
	public:
		void Initialize(const FRigMapperMathFeature& FeatureDefinition, const TMap<FName, FNodePtr>& Nodes);
		bool Evaluate(const FNodeCollection& NodeCollection, double& OutValue) const;

	private:
		struct FMathInput
		{
			FNodePtr Node;                // valid if connected to another node
			double ConstantValue = 0.0;   // used if Node name is empty, meaning constant value is set
		};

		ERigMapperMathOperation Operation;
		TArray<FMathInput, TInlineAllocator<3>> LinkedInputs;
	};
}

#undef UE_API
