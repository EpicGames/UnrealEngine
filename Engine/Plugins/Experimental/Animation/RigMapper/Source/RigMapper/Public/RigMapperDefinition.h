// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DataAsset.h"
#include "Engine/AssetUserData.h"

#include "RigMapperDefinition.generated.h"

#define UE_API RIGMAPPER_API


class FJsonObject;
class FJsonValue;
class URigMapperLinkedDefinitions;
class UEdGraph;

UENUM(BlueprintType)
enum class ERigMapperFeatureType : uint8
{
	Input,
	WeightedSum,
	SDK,
	Multiply,
	MathOp,
	Output,
	NullOutput,
	Invalid
};

UENUM(BlueprintType)
enum class ERigMapperMathOperation : uint8
{
	Min,
	Max,
	Abs,
	Negate,
	Floor,
	Ceil,
	Round,
	Divide,
	Multiply,
	Sum,
	Clamp,
	Lerp,
	Average
};

UENUM(BlueprintType)
enum class ERigMapperValidationSeverity : uint8
{
	Warning,
	Error
};

USTRUCT(BlueprintType)
struct FRigMapperValidationMessage
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadOnly, Category = "Animation|Rig Mapper")
	ERigMapperValidationSeverity Severity = ERigMapperValidationSeverity::Warning;

	UPROPERTY(BlueprintReadOnly, Category = "Animation|Rig Mapper")
	FString Message;

	// Optional: which definition in the chain caused it (INDEX_NONE for single definitions)
	UPROPERTY(BlueprintReadOnly, Category = "Animation|Rig Mapper")
	int32 DefinitionIndex = INDEX_NONE;
};

USTRUCT(BlueprintType)
struct FRigMapperValidationContext
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadOnly, Category = "Animation|Rig Mapper")
	TArray<FRigMapperValidationMessage> Messages;

	void AddWarning(const FString& Message, int32 DefinitionIndex = INDEX_NONE)
	{
		Messages.Add({ ERigMapperValidationSeverity::Warning, Message, DefinitionIndex });
	}

	void AddError(const FString& Message, int32 DefinitionIndex = INDEX_NONE)
	{
		Messages.Add({ ERigMapperValidationSeverity::Error, Message, DefinitionIndex });
	}

	int32 GetErrorCount() const
	{
		int32 Count = 0;
		for (const FRigMapperValidationMessage& Msg : Messages)
		{
			if (Msg.Severity == ERigMapperValidationSeverity::Error) { ++Count; }
		}
		return Count;
	}

	int32 GetWarningCount() const
	{
		int32 Count = 0;
		for (const FRigMapperValidationMessage& Msg : Messages)
		{
			if (Msg.Severity == ERigMapperValidationSeverity::Warning) { ++Count; }
		}
		return Count;
	}

	bool HasErrors() const { return GetErrorCount() > 0; }
	bool HasWarnings() const { return GetWarningCount() > 0; }

	// Flush all messages to UE_LOG
	UE_API void LogAll() const;
};

USTRUCT(BlueprintType)
struct FRigMapperFeature
{
	GENERATED_BODY()

public:
	using FBakedInput = TPair<TSharedPtr<FRigMapperFeature>, TArray<TSharedPtr<FRigMapperFeature>>>;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Animation|Rig Mapper", meta=(DisplayPriority=0))
	FString Name;

public:
	virtual ~FRigMapperFeature() = default;
	
	FRigMapperFeature() {}

	explicit FRigMapperFeature(const FString& InName) : Name(InName) {}

	virtual bool LoadFromJsonObject(const TSharedPtr<FJsonObject>& JsonObject) { return false; };

	UE_API virtual bool IsValid(const TArray<FString>& InputNames, bool bWarn = false, FRigMapperValidationContext* OutContext = nullptr) const;

	virtual ERigMapperFeatureType GetFeatureType() const { return ERigMapperFeatureType::Input; }

	virtual void GetInputs(TArray<FString>& OutInputs) const {}

	virtual bool BakeInput(URigMapperLinkedDefinitions* LinkedDefinitions, int32 DefinitionIndex, FBakedInput& OutBakedInput, FRigMapperValidationContext* OutContext = nullptr) const { return false; }

	virtual bool ContainsInput(const FString& InInput) const { return false; };

	bool operator==(const FRigMapperFeature& InOther) const
	{
		return Name == InOther.Name;
	}
	
protected:
	UE_API bool GetJsonArray(TSharedPtr<FJsonObject> JsonObject, TArray<TSharedPtr<FJsonValue>>& OutArray, const FString& Identifier, const FString& OwnerIdentifier="") const;
};

USTRUCT(BlueprintType)
struct FRigMapperMathInput
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Animation|Rig Mapper")
	FString NodeName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Animation|Rig Mapper")
	double ConstantValue = 0.0;

	bool operator==(const FRigMapperMathInput& InOther) const
	{
		return FMath::IsNearlyEqual(ConstantValue, InOther.ConstantValue, SMALL_NUMBER)
			&& NodeName == InOther.NodeName;
	}
};

USTRUCT(BlueprintType)
struct FRigMapperMathFeature : public FRigMapperFeature
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Animation|Rig Mapper", meta = (DisplayPriority = 1))
	ERigMapperMathOperation Operation = ERigMapperMathOperation::Min;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Animation|Rig Mapper", meta = (DisplayPriority = 2))
	TArray<FRigMapperMathInput> Inputs;

public:
	FRigMapperMathFeature() {}
	explicit FRigMapperMathFeature(const FString& InName) : FRigMapperFeature(InName) {}

	UE_API virtual bool LoadFromJsonObject(const TSharedPtr<FJsonObject>& JsonObject) override;
	UE_API virtual bool IsValid(const TArray<FString>& InputNames, bool bWarn = false, FRigMapperValidationContext* OutContext = nullptr) const override;
	virtual ERigMapperFeatureType GetFeatureType() const override { return ERigMapperFeatureType::MathOp; }
	UE_API virtual void GetInputs(TArray<FString>& OutInputs) const override;
	UE_API virtual bool BakeInput(URigMapperLinkedDefinitions* LinkedDefinitions, int32 DefinitionIndex, FBakedInput& OutBakedInput, FRigMapperValidationContext* OutContext = nullptr) const override;
	UE_API virtual bool ContainsInput(const FString& InInput) const override;

	static int32 GetMinInputCount(ERigMapperMathOperation Op)
	{
		switch (Op)
		{
		case ERigMapperMathOperation::Abs:
		case ERigMapperMathOperation::Negate:
		case ERigMapperMathOperation::Floor:
		case ERigMapperMathOperation::Ceil:
		case ERigMapperMathOperation::Round:
			return 1;
		case ERigMapperMathOperation::Min:
		case ERigMapperMathOperation::Max:
		case ERigMapperMathOperation::Average:
		case ERigMapperMathOperation::Divide:
		case ERigMapperMathOperation::Multiply:
		case ERigMapperMathOperation::Sum:
			return 2;
		case ERigMapperMathOperation::Clamp:
		case ERigMapperMathOperation::Lerp:
			return 3;
		default:
			return 1;
		}
	}

	static int32 GetMaxInputCount(ERigMapperMathOperation Op)
	{
		switch (Op)
		{
		case ERigMapperMathOperation::Abs:
		case ERigMapperMathOperation::Negate:
		case ERigMapperMathOperation::Floor:
		case ERigMapperMathOperation::Ceil:
		case ERigMapperMathOperation::Round:
			return 1;
		case ERigMapperMathOperation::Clamp:
		case ERigMapperMathOperation::Lerp:
			return 3;
		case ERigMapperMathOperation::Min:
		case ERigMapperMathOperation::Max:
		case ERigMapperMathOperation::Average:
		case ERigMapperMathOperation::Divide:
		case ERigMapperMathOperation::Multiply:
		case ERigMapperMathOperation::Sum:
			return INT32_MAX;
		default:
			return INT32_MAX;
		}
	}

	bool operator==(const FRigMapperMathFeature& InOther) const
	{
		if (Operation != InOther.Operation)
		{
			return false;
		}
		if (Inputs.Num() != InOther.Inputs.Num())
		{
			return false;
		}

		for (int32 i = 0; i < Inputs.Num(); ++i)
		{
			if (Inputs[i] != InOther.Inputs[i])
			{
				return false;
			}
		}

		return FRigMapperFeature::operator==(InOther);
	}
};

USTRUCT(BlueprintType)
struct FRigMapperMultiplyFeature : public FRigMapperFeature
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Animation|Rig Mapper", meta=(DisplayPriority=1))
	TArray<FString> Inputs;
	
public:
	FRigMapperMultiplyFeature() {}

	explicit FRigMapperMultiplyFeature(const FString& InName) : FRigMapperFeature(InName) {}

	UE_API virtual bool LoadFromJsonObject(const TSharedPtr<FJsonObject>& JsonObject) override;

	UE_API virtual bool IsValid(const TArray<FString>& InputNames, bool bWarn = false, FRigMapperValidationContext* OutContext = nullptr) const override;

	virtual ERigMapperFeatureType GetFeatureType() const override { return ERigMapperFeatureType::Multiply; }

	UE_API virtual void GetInputs(TArray<FString>& OutInputs) const override { OutInputs = Inputs; }

	UE_API virtual bool BakeInput(URigMapperLinkedDefinitions* LinkedDefinitions, int32 DefinitionIndex, FBakedInput& OutBakedInput, FRigMapperValidationContext* OutContext = nullptr) const override;

	UE_API virtual bool ContainsInput(const FString& InInput) const override;

	bool operator==(const FRigMapperMultiplyFeature& InOther) const
	{
		return FRigMapperFeature::operator==(InOther) && Inputs == InOther.Inputs;
	}
};

USTRUCT(BlueprintType)
struct FRigMapperFeatureRange
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Animation|Rig Mapper")
	bool bHasLowerBound = false;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Animation|Rig Mapper")
	double LowerBound = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Animation|Rig Mapper")
	bool bHasUpperBound = false;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Animation|Rig Mapper")
	double UpperBound = 0;

	bool operator==(const FRigMapperFeatureRange& InOther) const
	{
		return bHasLowerBound == InOther.bHasLowerBound && FMath::IsNearlyEqual(LowerBound, InOther.LowerBound, SMALL_NUMBER)
			&& bHasUpperBound == InOther.bHasUpperBound && FMath::IsNearlyEqual(UpperBound, InOther.UpperBound, SMALL_NUMBER);
	}
};	

USTRUCT(BlueprintType)
struct FRigMapperWsFeature : public FRigMapperFeature
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TMap<FString, double> Inputs;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Animation|Rig Mapper", meta=(DisplayPriority=1))
	FRigMapperFeatureRange Range;
	
public:
	FRigMapperWsFeature() {}

	explicit FRigMapperWsFeature(const FString& InName) : FRigMapperFeature(InName) {}

	UE_API virtual bool LoadFromJsonObject(const TSharedPtr<FJsonObject>& JsonObject) override;

	UE_API virtual bool IsValid(const TArray<FString>& InputNames, bool bWarn = false, FRigMapperValidationContext* OutContext = nullptr) const override;

	virtual ERigMapperFeatureType GetFeatureType() const override { return ERigMapperFeatureType::WeightedSum; }

	UE_API virtual void GetInputs(TArray<FString>& OutInputs) const override;

	UE_API virtual bool BakeInput(URigMapperLinkedDefinitions* LinkedDefinitions, int32 DefinitionIndex, FBakedInput& OutBakedInput, FRigMapperValidationContext* OutContext = nullptr) const override;

	UE_API virtual bool ContainsInput(const FString& InInput) const override;

	bool operator==(const FRigMapperWsFeature& InOther) const
	{
		if (Inputs.Num() != InOther.Inputs.Num())
		{
			return false; 
		}

		for (const TPair<FString, double>& Pair : Inputs)
		{
			const double* OtherValue = InOther.Inputs.Find(Pair.Key);
			if (OtherValue == nullptr || !FMath::IsNearlyEqual(Pair.Value, *OtherValue, SMALL_NUMBER))
			{
				return false; 
			}
		}

		return FRigMapperFeature::operator==(InOther) && Range == InOther.Range;
	}
};

USTRUCT(BlueprintType)
struct FRigMapperSdkKey
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Animation|Rig Mapper")
	double In = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Animation|Rig Mapper")
	double Out = 0;

	bool operator==(const FRigMapperSdkKey& InOther) const
	{
		return FMath::IsNearlyEqual(In, InOther.In, SMALL_NUMBER) && FMath::IsNearlyEqual(Out, InOther.Out, SMALL_NUMBER);
	}
};

USTRUCT(BlueprintType)
struct FRigMapperSdkFeature : public FRigMapperFeature
{
	GENERATED_BODY()
	
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Animation|Rig Mapper", meta=(DisplayPriority=1))
	FString Input;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Animation|Rig Mapper", meta=(DisplayPriority=1))
	TArray<FRigMapperSdkKey> Keys;
	
public:
	FRigMapperSdkFeature() {}

	explicit FRigMapperSdkFeature(const FString& InName) : FRigMapperFeature(InName) {};

	UE_API virtual bool LoadFromJsonObject(const TSharedPtr<FJsonObject>& JsonObject) override;
	
	UE_API virtual bool IsValid(const TArray<FString>& InputNames, bool bWarn = false, FRigMapperValidationContext* OutContext = nullptr) const override;

	virtual ERigMapperFeatureType GetFeatureType() const override { return ERigMapperFeatureType::SDK; }

	virtual void GetInputs(TArray<FString>& OutInputs) const override { OutInputs = { Input }; };

	UE_API virtual bool BakeInput(URigMapperLinkedDefinitions* LinkedDefinitions, int32 DefinitionIndex, FBakedInput& OutBakedInput, FRigMapperValidationContext* OutContext = nullptr) const override;

	UE_API virtual bool ContainsInput(const FString& InInput) const override { return Input == InInput; };

	static UE_API bool BakeKeys(const FRigMapperSdkFeature& InSdk, const FRigMapperSdkFeature& OutSdk, TArray<FRigMapperSdkKey>& BakedKeys);

	bool operator==(const FRigMapperSdkFeature& InOther) const
	{
		return FRigMapperFeature::operator==(InOther) && Input == InOther.Input && Keys == InOther.Keys;
	}
};

USTRUCT(BlueprintType)
struct FRigMapperFeatureDefinitions
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Animation|Rig Mapper")
	TArray<FRigMapperMultiplyFeature> Multiply;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Animation|Rig Mapper")
	TArray<FRigMapperWsFeature> WeightedSums;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Animation|Rig Mapper")
	TArray<FRigMapperSdkFeature> SDKs;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Animation|Rig Mapper")
	TArray<FRigMapperMathFeature> MathOps;

public:
	UE_API bool AddFromJsonObject(const FString& FeatureName, const TSharedPtr<FJsonObject>& JsonObject);

	UE_API bool GetFeatureNames(TArray<FString>& OutFeatureNames) const;
	
	UE_API bool IsValid(const TArray<FString>& InputNames, bool bWarn = false, FRigMapperValidationContext* OutContext = nullptr) const;

	UE_API void Empty();
	
	UE_API FRigMapperFeature* Find(const FString& FeatureName, ERigMapperFeatureType& OutFeatureType);

	bool operator==(const FRigMapperFeatureDefinitions& InOther) const
	{
		return Multiply == InOther.Multiply && WeightedSums == InOther.WeightedSums && SDKs == InOther.SDKs && MathOps == InOther.MathOps;
	}
};


DECLARE_MULTICAST_DELEGATE(FOnRigMapperDefinitionCreated);
DECLARE_MULTICAST_DELEGATE(FOnRigMapperDefinitionUpdated);

UCLASS(MinimalAPI, BlueprintType)
class URigMapperDefinition : public UDataAsset
{
	GENERATED_BODY()

public:
	// Delegate to notify listeners that we have loaded definitions
	FOnRigMapperDefinitionCreated OnRigMapperDefinitionCreated;
	// Delegate to notify listeners that the definition is modified (features added or deleted)
	FOnRigMapperDefinitionUpdated OnRigMapperDefinitionUpdated;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Animation|Rig Mapper")
	TArray<FString> Inputs;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Animation|Rig Mapper")
	FRigMapperFeatureDefinitions Features;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Animation|Rig Mapper")
	TMap<FString, FString> Outputs;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Animation|Rig Mapper")
	TArray<FString> NullOutputs;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="Animation|Rig Mapper")
	bool bValidated = false;

#if WITH_EDITORONLY_DATA
	/** Graph used by the definition editor. */
	UPROPERTY()
	TObjectPtr<UEdGraph> EditorGraph;
#endif
	
private:
	UPROPERTY()
	FString JsonSource;

public:
	UFUNCTION(BlueprintCallable, Category="Animation|Rig Mapper")
	UE_API bool LoadFromJsonFile(const FFilePath& JsonFilePath);
	
	UFUNCTION(BlueprintCallable, Category="Animation|Rig Mapper")
	UE_API bool LoadFromJsonString(const FString& JsonString);
	
	UFUNCTION(BlueprintCallable, Category="Animation|Rig Mapper")
	UE_API bool ExportAsJsonString(FString& OutJsonString) const;

	UFUNCTION(BlueprintCallable, Category="Animation|Rig Mapper")
	UE_API bool ExportAsJsonFile(const FFilePath& JsonFilePath) const;

	UFUNCTION(BlueprintCallable, Category = "Animation|Rig Mapper")
	UE_API bool LoadInputsFromSkeletalMesh(const TArray<FName>& InNewInputCurves);

	UFUNCTION(BlueprintCallable, Category = "Animation|Rig Mapper")
	UE_API bool LoadOutputsFromSkeletalMesh(const TArray<FName>& InNewOutputCurves);
	
	UFUNCTION(BlueprintCallable, Category="Animation|Rig Mapper")
	UE_API void Empty();

	UFUNCTION(BlueprintCallable, Category = "Animation|Rig Mapper")
	UE_API bool IsDefinitionValid(bool bWarn = false, bool bForce = false) const;
	UE_API bool IsDefinitionValid(FRigMapperValidationContext* OutContext, bool bWarn = false, bool bForce = false) const;

	UFUNCTION(BlueprintCallable, Category="Animation|Rig Mapper")
	UE_API bool Validate();
	UE_API bool Validate(FRigMapperValidationContext* OutContext);
	UE_API void LogAll(const FRigMapperValidationContext& InContext) const;

	UFUNCTION(BlueprintCallable, Category = "Animation|Rig Mapper")
	UE_API bool WasDefinitionValidated() const;

private:
	bool LoadInputsFromJsonObject(const TSharedPtr<FJsonObject>& JsonObject);
	bool LoadFeaturesFromJsonObject(const TSharedPtr<FJsonObject>& JsonObject, TArray<FString>& FeatureNames);
	bool LoadOutputsFromJsonObject(const TSharedPtr<FJsonObject>& JsonObject, const TArray<FString>& FeatureNames);
	bool LoadNullOutputsFromJsonObject(const TSharedPtr<FJsonObject>& JsonObject);
	void SetDefinitionValid(bool bValid);

#if WITH_EDITOR
public:
	/**
	 * Only clears definition from the cache.
	 */
	UE_API virtual void InvalidateCache();
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	/**
	 * Similar to PostEditChangeProperty but for Definition editor graph wrapper changes.
	 */
	UE_API virtual void BroadcastDefinitionModified();

private:	
	UE_API virtual EDataValidationResult IsDataValid(FDataValidationContext& Context) const override;
#endif
	
	// todo: helper function to generate new asset from json file
	// todo: invalidate on property changed
};

UCLASS(MinimalAPI, BlueprintType)
class URigMapperLinkedDefinitions : public UDataAsset
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Animation|Rig Mapper")
	TArray<TObjectPtr<URigMapperDefinition>> SourceDefinitions;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Animation|Rig Mapper")
	TObjectPtr<URigMapperDefinition> BakedDefinition;
	
public:
	// todo: bake to new
	// todo: invalidate on property or source definition changed 
	
	UFUNCTION(BlueprintCallable, Category="Animation|Rig Mapper")
	UE_API bool BakeDefinitions();
	UE_API bool BakeDefinitions(FRigMapperValidationContext* OutContext);

	UFUNCTION(BlueprintCallable, Category="Animation|Rig Mapper")
	UE_API bool Validate();
	UE_API bool Validate(FRigMapperValidationContext* OutContext);

	UFUNCTION(BlueprintCallable, Category="Animation|Rig Mapper")
	UE_API bool AreLinkedDefinitionsValid() const;
	UE_API bool AreLinkedDefinitionsValid(FRigMapperValidationContext* OutContext) const;

	UE_API bool GetBakedInputRec(const FString& InputName, const int32 DefinitionIndex, FRigMapperFeature::FBakedInput& OutBakedInput, bool& bOutMissingIsNotNullOutput, FRigMapperValidationContext* OutContext);
	UE_API void LogAll(const FRigMapperValidationContext& InContext) const;

private:
	bool AddBakedInputFeature(const TSharedPtr<FRigMapperFeature>& Feature) const;
	
	void AddBakedInputs(const TArray<FRigMapperFeature::FBakedInput>& BakedInputs, const TArray<int32>& BakedOutputIndices, const TArray<TPair<FString, FString>>& PairedOutputs) const;
	// todo: helper functions to bake to new / existing asset
	// todo: helper function to import/export from/to json files

#if WITH_EDITOR
	UE_API virtual EDataValidationResult IsDataValid(FDataValidationContext& Context) const override;
#endif
};

UCLASS(MinimalAPI, NotBlueprintable, HideCategories = (Object))
class URigMapperDefinitionUserData : public UAssetUserData
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Animation|Rig Mapper")
	TArray<TObjectPtr<URigMapperDefinition>> Definitions;
};

#undef UE_API
