// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"

RIGMAPPER_API DECLARE_LOG_CATEGORY_EXTERN(LogRigMapper, Log, All);
