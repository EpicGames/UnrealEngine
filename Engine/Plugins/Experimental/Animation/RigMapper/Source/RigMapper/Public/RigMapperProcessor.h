// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigMapper.h"
#include "Misc/FrameTime.h"
#include "RigMapperProcessor.generated.h"

#define UE_API RIGMAPPER_API

DECLARE_STATS_GROUP(TEXT("RigMapper"), STATGROUP_RigMapper, STATCAT_Advanced);

DECLARE_CYCLE_STAT_EXTERN(TEXT("Get Rig Mapper"), STAT_RigMapperGetRigMapper, STATGROUP_RigMapper, RIGMAPPER_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Load Definition"), STAT_RigMapperLoadDefinition, STATGROUP_RigMapper, RIGMAPPER_API);


/**
 * A helper struct to gain performance when doing some batch remapping with the Rig Mapper pipeline
 * Useful when remapping frames every tick, or numerous frames at once, or when using chained definitions...
 * EvaluateFrame(s) methods work best if given the same set of CurveNames at every call 
 * Also provides a static utility to load rig mappers from definitions
 */
USTRUCT()
struct FRigMapperProcessor
{
	GENERATED_BODY()
		
	using FPose = TMap<FString, float>;
	using FPoseValues = TArray<TOptional<float>>;
	using FSparseBakedCurves = TMap<FFrameTime, FPoseValues>;

	FRigMapperProcessor() = default;
	UE_API explicit FRigMapperProcessor(const TArray<URigMapperDefinition*>& InDefinitions);
	UE_API explicit FRigMapperProcessor(URigMapperDefinition* InDefinition);

	/** @returns whether we have at least one rig mapper initialized, and all rig mappers are valid */
	UE_API bool IsValid() const;

	/** @returns the input names expected by the first rig mapper */
	UE_API const TArray<FName>& GetInputNames() const;

	/** @returns the output names given by the last rig mapper */
	UE_API const TArray<FName>& GetOutputNames() const;

	/** Evaluates a set of frames all at once - retrieve and/or cache curve names using @GetOutputNames for performance */
	UE_API bool EvaluateFrames(const TArray<FName>& CurveNames, const TArray<FPoseValues>& InFrameValues, TArray<FPoseValues>& OutFrameValues);

	/** Evaluates a set of frames all at once */
	UE_API bool EvaluateFrames(const TArray<FName>& CurveNames, const TArray<FPoseValues>& InFrameValues, TArray<FName>& OutCurveNames, TArray<FPoseValues>& OutFrameValues);

	/** Evaluates a set of frames all at once, interpolating between them using the given frame times */
	UE_API bool EvaluateFrames_Interp(const TArray<FName>& CurveNames, const TArray<FPoseValues>& InFrameValues, TArray<FPoseValues>& OutFrameValues, const TArray<FFrameTime>& FrameTimes);

	/** Evaluates a single frame - retrieve and/or cache curve names using @GetOutputNames for performance */
	UE_API bool EvaluateFrame(const TArray<FName>& CurveNames, const FPoseValues& InCurveValues, FPoseValues& OutCurveValues);

	/** Evaluates a single frame */
	UE_API bool EvaluateFrame(const TArray<FName>& CurveNames, const FPoseValues& InCurveValues, TArray<FName>& OutCurveNames, FPoseValues& OutCurveValues);

private:
	/** Evaluates a single frame for a single rig mapper */
	UE_API bool EvaluateFrame_Internal(const TArray<FName>& CurveNames, const FPoseValues& InCurveValues, FPoseValues& OutCurveValues, int32 RigIndex);
	
private:
	using FIndexCache = TArray<int32>;

	/** The rig mappers loaded from definitions */
	TArray<FacialRigMapping::FRigMapper> RigMappers;

	/** An index map for each rig mapper's input - if the input curve names change between two calls, it will be rebuilt */
	TArray<FIndexCache> IndexCache;
};

/*
* A singleton class which stores a cache of FRigMappers initialized from URigMapperDefinitions, for speed
*/
class FRigMapperDefinitionsSingleton
{
public:
	static FRigMapperDefinitionsSingleton& Get();
	bool GetRigMapper(const TObjectPtr<const URigMapperDefinition>& InDefinition, FacialRigMapping::FRigMapper& OutRigMapper);

	/* clear rig mapper from the cache if it exists*/
	void ClearFromCache(const TObjectPtr<const URigMapperDefinition>& InDefinition);
private:
	FRigMapperDefinitionsSingleton() = default;
	virtual ~FRigMapperDefinitionsSingleton() = default;
	TMap<FSoftObjectPath, FacialRigMapping::FRigMapper> RigMappers;
};

#undef UE_API
