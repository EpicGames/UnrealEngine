// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API RIGMAPPER_API

enum class ERigMapperFeatureType : uint8;

class FRigMapperUtils
{
public:
	/**
	 * Generates a unique name given a base one and a list of existing ones, by appending a type prefix and an index suffix.
	 *
	 * @param InExistingNames	List of existing feature names including inputs or outputs, depending on the type.
	 * @param InDesiredName		The desired Name. Note, generates '{type name}:node' if desired name is empty.
	 * @param InFeatureType		Type of the feature being generated to include/replace as a prefix.
	 * @return					Returns the unique name.
	 */
	static UE_API FString GenerateUniqueFeatureName(const TArray<FString>& InExistingNames, FString InDesiredName, ERigMapperFeatureType InFeatureType);
};

#undef UE_API
