// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class RigMapper : ModuleRules
{
	public RigMapper(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"ControlRig",
				"AnimationCore",
				"Json"
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
			});
	}
}
