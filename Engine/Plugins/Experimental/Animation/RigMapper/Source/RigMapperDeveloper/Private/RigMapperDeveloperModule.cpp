// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigMapperDeveloperModule.h"

#include "Modules/ModuleManager.h"

void FRigMapperDeveloperModule::StartupModule()
{
}

void FRigMapperDeveloperModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FRigMapperDeveloperModule, RigMapperDeveloper)
