// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

/**
 * FRigMapperDeveloperModule
 * Module for the developer only content in the RigMapper plugin
 */
class FRigMapperDeveloperModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface
};
