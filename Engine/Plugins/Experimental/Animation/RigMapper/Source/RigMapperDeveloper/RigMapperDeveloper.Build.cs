// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class RigMapperDeveloper : ModuleRules
{
	public RigMapperDeveloper(ReadOnlyTargetRules Target) : base(Target)
	{
			// Copying some of these from RigMapper.build.cs, our deps are likely leaner
			// and therefore these could be pruned if needed:
  
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"ControlRig",
				"AnimationCore",
				"Json",
				"RigMapper"
			});
	}
}
