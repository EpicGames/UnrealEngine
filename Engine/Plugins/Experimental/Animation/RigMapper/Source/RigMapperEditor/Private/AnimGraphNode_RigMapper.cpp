// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_RigMapper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_RigMapper)

#define LOCTEXT_NAMESPACE "RigMapper"

UAnimGraphNode_RigMapper::UAnimGraphNode_RigMapper(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

FText UAnimGraphNode_RigMapper::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("RigMapper", "Rig Mapper");
}

FText UAnimGraphNode_RigMapper::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_RigMapper_ToolTip", "Evaluates an output pose from the current pose using a control rig backward solve and a set of operations configured in a Json file");
}

FText UAnimGraphNode_RigMapper::GetMenuCategory() const
{
	return LOCTEXT("AnimGraphNode_RigMapper_Category", "Animation|Poses");
}

FLinearColor UAnimGraphNode_RigMapper::GetNodeBodyTintColor() const
{
	return FLinearColor(FColor::Emerald);
}

void UAnimGraphNode_RigMapper::ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton,
	FCompilerResultsLog& MessageLog)
{
	// todo: validate config
	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);
}

FEditorModeID UAnimGraphNode_RigMapper::GetEditorMode() const
{
	// todo: override to set a custom editor mode
	return Super::GetEditorMode();
}

EAnimAssetHandlerType UAnimGraphNode_RigMapper::SupportsAssetClass(const UClass* AssetClass) const
{
	// todo: Rig Mapper Config Asset
	// if (AssetClass->IsChildOf(URigMapperConfig::StaticClass()))
	// {
	// 	return EAnimAssetHandlerType::Supported;
	// }
	// else
	// {
	// 	return EAnimAssetHandlerType::NotSupported;
	// }

	return EAnimAssetHandlerType::NotSupported;
}

void UAnimGraphNode_RigMapper::CopyNodeDataToPreviewNode(FAnimNode_Base* InPreviewNode)
{
	// todo: implement to transfer input/output to preview?
}

void UAnimGraphNode_RigMapper::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimNode_RigMapper, Definitions))
	{
		USkeletalMesh* TargetMesh = nullptr;
		if (const UAnimBlueprint* AnimBlueprint = GetAnimBlueprint())
		{
			TargetMesh = AnimBlueprint->GetPreviewMesh();
			if (!TargetMesh && AnimBlueprint->TargetSkeleton)
			{
				TargetMesh = AnimBlueprint->TargetSkeleton->GetPreviewMesh();
			}
		}
		Node.InitializeRigMapping(TargetMesh);
		ReconstructNode();
	}
}

#undef LOCTEXT_NAMESPACE
