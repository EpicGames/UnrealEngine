// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigMapperDefinitionAssetTypeActions.h"

#include "RigMapperDefinition.h"
#include "RigMapperDefinitionEditorToolkit.h"

#define LOCTEXT_NAMESPACE "RigMapperDefinitionAssetTypeActions"

UClass* FRigMapperDefinitionAssetTypeActions::GetSupportedClass() const
{
	return URigMapperDefinition::StaticClass();
}

FText FRigMapperDefinitionAssetTypeActions::GetName() const
{
	return LOCTEXT("FRigMapperDefinitionAssetTypeActionsName", "Rig Mapper Definition");
}

FColor FRigMapperDefinitionAssetTypeActions::GetTypeColor() const
{
	return FColor::Yellow;
}

uint32 FRigMapperDefinitionAssetTypeActions::GetCategories()
{
	return EAssetTypeCategories::Animation;
}

void FRigMapperDefinitionAssetTypeActions::OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<IToolkitHost> ToolkitHost)
{
	const EToolkitMode::Type Mode = ToolkitHost.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;

	for (UObject* Obj : InObjects)
	{
		if (URigMapperDefinition* Definition = Cast<URigMapperDefinition>(Obj))
		{
			const TSharedRef<FRigMapperDefinitionEditorToolkit> EditorToolkit = MakeShareable(new FRigMapperDefinitionEditorToolkit);
			EditorToolkit->Initialize(Definition, Mode, ToolkitHost);
		}
	}
}

#undef LOCTEXT_NAMESPACE