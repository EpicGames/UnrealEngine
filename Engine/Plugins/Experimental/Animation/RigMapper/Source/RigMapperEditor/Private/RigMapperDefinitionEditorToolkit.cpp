// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigMapperDefinitionEditorToolkit.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "RigMapperGraph/RigMapperDefinitionEditorGraph.h"
#include "RigMapperGraph/RigMapperDefinitionEditorGraphNode.h"
#include "RigMapperGraph/SRigMapperDefinitionGraphEditor.h"
#include "RigMapperGraph/SRigMapperValidationBanner.h"
#include "RigMapperEditorModule.h"

#include "DetailsViewObjectFilter.h"
#include "PropertyEditorModule.h"
#include "Modules/ModuleManager.h"
#include "RigMapperGraph/SRigMapperDefinitionStructureView.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Docking/SDockTab.h"
#include "Logging/MessageLog.h"

#define LOCTEXT_NAMESPACE "RigMapperDefinitionEditorToolkit"

const FName FRigMapperDefinitionEditorToolkit::DefinitionEditorGraphTabId("RigMapperEditor_DefinitionGraphView");
const FName FRigMapperDefinitionEditorToolkit::DefinitionEditorStructureTabId("RigMapperEditor_DefinitionStructureView");
const FName FRigMapperDefinitionEditorToolkit::DefinitionEditorDetailsTabId("RigMapperEditor_DefinitionDetailsView");

const TMap<FName, ERigMapperFeatureType> FRigMapperDefinitionEditorToolkit::PropertyNameToNodeTypeMapping = {
	{ GET_MEMBER_NAME_CHECKED(URigMapperDefinition, Inputs), ERigMapperFeatureType::Input },
	{ GET_MEMBER_NAME_CHECKED(URigMapperDefinition, Features), ERigMapperFeatureType::Invalid },
	{ GET_MEMBER_NAME_CHECKED(FRigMapperFeatureDefinitions, Multiply), ERigMapperFeatureType::Multiply },
	{ GET_MEMBER_NAME_CHECKED(FRigMapperFeatureDefinitions, MathOps), ERigMapperFeatureType::MathOp },
	{ GET_MEMBER_NAME_CHECKED(FRigMapperFeatureDefinitions, WeightedSums), ERigMapperFeatureType::WeightedSum },
	{ GET_MEMBER_NAME_CHECKED(FRigMapperFeatureDefinitions, SDKs), ERigMapperFeatureType::SDK },
	{ GET_MEMBER_NAME_CHECKED(URigMapperDefinition, Outputs), ERigMapperFeatureType::Output },
	{ GET_MEMBER_NAME_CHECKED(URigMapperDefinition, NullOutputs), ERigMapperFeatureType::NullOutput },
};

void FRigMapperDefinitionEditorToolkit::Initialize(URigMapperDefinition* InDefinition, EToolkitMode::Type InMode, TSharedPtr<IToolkitHost> InToolkitHost)
{
	Definition = InDefinition;
	if (Definition)
	{
		URigMapperDefinitionEditorGraph* Graph = Cast<URigMapperDefinitionEditorGraph>(Definition->EditorGraph);
		if (!Graph)
		{
			Graph = NewObject<URigMapperDefinitionEditorGraph>(Definition, NAME_None, RF_Transactional);
			Definition->EditorGraph = Graph;
			Definition->MarkPackageDirty();
			Graph->Initialize(Definition);
			Graph->RebuildGraph();
			Graph->LayoutNodes();
		}
		else
		{
			Graph->Initialize(Definition);
			Graph->RebuildNodeMaps();
		}
	}
	
	// todo
	// GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetPostImport.AddSP(this, &FSimpleAssetEditor::HandleAssetPostImport);
	// FCoreUObjectDelegates::OnObjectsReplaced.AddSP(this, &FSimpleAssetEditor::OnObjectsReplaced);
	
	const TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout("Standalone_RigMapperDefinitionEditor_Layout_v1")
	->AddArea
	(
		FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
		->Split
		(
			FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewStack()
				->AddTab(DefinitionEditorStructureTabId, ETabState::OpenedTab)
				->SetSizeCoefficient(0.2f)
			)
			->Split
			(
				FTabManager::NewStack() 
				->SetHideTabWell(true)
				->AddTab(DefinitionEditorGraphTabId, ETabState::OpenedTab)
				->SetSizeCoefficient(0.6f)
			)
			->Split
			(
				FTabManager::NewStack() 
				->AddTab(DefinitionEditorDetailsTabId, ETabState::OpenedTab)
				->SetSizeCoefficient(0.2f)
			)
		)
	);

	InitAssetEditor(
		InMode,
		InToolkitHost,
		FRigMapperEditorModule::AppIdentifier,
		Layout,
		true /*bCreateDefaultStandaloneMenu*/,
		true /*bCreateDefaultToolbar*/,
		InDefinition
	);

	FRigMapperEditorModule::RegisterRigMapperDefinitionToolbarEntries();
}

void FRigMapperDefinitionEditorToolkit::OnClose()
{

	if (Definition)
	{
		// Sync graph state back to Definition before closing
		if (URigMapperDefinitionEditorGraph* Graph = Cast<URigMapperDefinitionEditorGraph>(Definition->EditorGraph))
		{
			Graph->ApplyToDefinition();
			Graph->OnGraphStructureUpdated.RemoveAll(this);

			// Validate new definition
			FRigMapperValidationContext Context;
			const bool bValid = Definition->Validate(&Context);
			Definition->InvalidateCache();

			// Log to output log
			Definition->LogAll(Context);

			// Open MessageLog for errors, if any
			if (Context.HasErrors())
			{
				FMessageLog MessageLog("RigMapperEditor");
				for (const FRigMapperValidationMessage& Msg : Context.Messages)
				{
					if (Msg.Severity == ERigMapperValidationSeverity::Error)
					{
						MessageLog.Error(FText::FromString(Msg.Message));
					}
					else
					{
						MessageLog.Warning(FText::FromString(Msg.Message));
					}
				}
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("ObjectName"), FText::FromString(Definition->GetName()));
				MessageLog.Error(FText::Format(LOCTEXT("RigMapperEditorActions_ValidateFailed", "Failed to validate definition \"{ObjectName}\". See output log for more details"), Arguments));
				MessageLog.Open(EMessageSeverity::Error);
			}
		}
	}
	if (DetailsView)
	{
		DetailsView->OnFinishedChangingProperties().Clear();
	}
}

void FRigMapperDefinitionEditorToolkit::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("RigMapperDefinitionEditorTabGroup", "Rig Mapper Definition Editor"));

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);
	
	InTabManager->RegisterTabSpawner(DefinitionEditorGraphTabId, FOnSpawnTab::CreateSP(this, &FRigMapperDefinitionEditorToolkit::SpawnGraphTab))
		.SetDisplayName(LOCTEXT("RigMapperDefinitionEditorGraphViewName","Graph"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x"));

	InTabManager->RegisterTabSpawner(DefinitionEditorStructureTabId, FOnSpawnTab::CreateSP(this, &FRigMapperDefinitionEditorToolkit::SpawnStructureTab))
		.SetDisplayName(LOCTEXT("RigMapperDefinitionEditorStructureViewName", "Structure"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.Palette"));

	InTabManager->RegisterTabSpawner(DefinitionEditorDetailsTabId, FOnSpawnTab::CreateSP(this, &FRigMapperDefinitionEditorToolkit::SpawnDetailsTab))
		.SetDisplayName(LOCTEXT("RigMapperDefinitionEditorDetailsViewName", "Details"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));
}

void FRigMapperDefinitionEditorToolkit::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	InTabManager->UnregisterTabSpawner(DefinitionEditorGraphTabId);
	InTabManager->UnregisterTabSpawner(DefinitionEditorStructureTabId);
	InTabManager->UnregisterTabSpawner(DefinitionEditorDetailsTabId);
}

FName FRigMapperDefinitionEditorToolkit::GetToolkitFName() const
{
	return FName("RigMapperDefinitionEditor");
}

FText FRigMapperDefinitionEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("RigMapperDefinitionEditorToolkitBaseToolkitName", "Rig Mapper Definition Editor");
}

FString FRigMapperDefinitionEditorToolkit::GetWorldCentricTabPrefix() const
{
	return "Rig Mapper Definition ";
}

FLinearColor FRigMapperDefinitionEditorToolkit::GetWorldCentricTabColorScale() const
{
	return FLinearColor( 0.0f, 0.1f, 0.2f, 0.5f );
}

void FRigMapperDefinitionEditorToolkit::SaveAsset_Execute()
{
	if (GraphEditor)
	{
		if (URigMapperDefinitionEditorGraph* Graph =
			Cast<URigMapperDefinitionEditorGraph>(GraphEditor->GetEditorGraphObject()))
		{
			Graph->ApplyToDefinition();

			if (Definition)
			{
				FRigMapperValidationContext Context;
				const bool bValid = Definition->Validate(&Context);
				Definition->InvalidateCache();

				// Log to output log
				Definition->LogAll(Context);
				if (TSharedPtr<SRigMapperValidationBanner> Banner = GraphEditor->GetValidationBanner())
				{
					Banner->SetValidationResult(Context);
				}
			}
		}
	}

	// Standard save
	FAssetEditorToolkit::SaveAsset_Execute();
}

TSharedRef<SDockTab> FRigMapperDefinitionEditorToolkit::SpawnGraphTab(const FSpawnTabArgs& SpawnTabArgs)
{
	check(Definition);
	// todo: search and replace (easy rename inputs, outputs and features)
	// todo: error log
	GraphEditor = SNew(SRigMapperDefinitionGraphEditor, Definition);

	GraphEditor->OnSelectionChanged.BindRaw(this, &FRigMapperDefinitionEditorToolkit::HandleGraphSelectionChanged);
	if (URigMapperDefinitionEditorGraph* Graph =
		Cast<URigMapperDefinitionEditorGraph>(Definition->EditorGraph))
	{
		Graph->OnGraphStructureUpdated.AddRaw(this, &FRigMapperDefinitionEditorToolkit::HandleGraphStructureUpdated);
	}
	GraphTab = SNew(SDockTab)
	.TabColorScale(GetTabColorScale())
	.Label(LOCTEXT("RigMapperDefinitionEditorToolkitGraphTab", "Graph"))
	[
		GraphEditor.ToSharedRef()
	];

	return GraphTab.ToSharedRef();
}


void FRigMapperDefinitionEditorToolkit::HandleGraphStructureUpdated()
{
	if (StructureView)
	{
		StructureView->RebuildTree();
	}
}

TSharedRef<IDetailCustomization> FRigMapperDefinitionEditorToolkit::FDetailsViewCustomization::MakeInstance()
{
	return MakeShared<FDetailsViewCustomization>();
}

void FRigMapperDefinitionEditorToolkit::FDetailsViewCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailLayout.GetObjectsBeingCustomized(Objects);
	
	DetailLayout.EditCategory("Animation|Rig Mapper", LOCTEXT("RigMapperDefinitionElements", "Elements"));
	DetailLayout.HideCategory("Animation");
	DetailLayout.HideCategory("Rig Mapper");
}

TSharedRef<SDockTab> FRigMapperDefinitionEditorToolkit::SpawnDetailsTab(const FSpawnTabArgs& SpawnTabArgs)
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bAllowMultipleTopLevelObjects = false;

	DetailsView = PropertyModule.CreateDetailView(DetailsViewArgs);

	if (Definition)
	{
		Definition->SetFlags(RF_Transactional);
	}

	TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabColorScale(GetTabColorScale())
		.Label(LOCTEXT("ToolkitDetailsTab", "Details"))
		[
			DetailsView.ToSharedRef()
		];

	return DockTab;
}

TSharedRef<SDockTab> FRigMapperDefinitionEditorToolkit::SpawnStructureTab(const FSpawnTabArgs& SpawnTabArgs)
{
	check(Definition);
	URigMapperDefinitionEditorGraph* Graph = Cast<URigMapperDefinitionEditorGraph>(Definition->EditorGraph);
	StructureView = SNew(SRigMapperDefinitionStructureView, Graph);
	StructureView->OnSelectionChanged.BindRaw(this, &FRigMapperDefinitionEditorToolkit::HandleStructureSelectionChanged);
		
	TSharedRef<SDockTab> DockTab = SNew(SDockTab)
	.TabColorScale(GetTabColorScale())
	.Label(LOCTEXT("RigMapperDefinitionEditorToolkitStructureTab", "Structure"))
	[
		StructureView.ToSharedRef()
	];

	return DockTab;
}


bool FRigMapperDefinitionEditorToolkit::HandleIsPropertyVisible(const FPropertyAndParent& PropertyAndParent)
{
	const int32 ArrayIndex = PropertyAndParent.ArrayIndex;
	const FName PropertyName = PropertyAndParent.Property.GetFName();
	
	if (StructureView->IsSelectionEmpty())
	{
		return true;
	}
	
	if (PropertyAndParent.ParentProperties.Num() > 2)
	{
		return true;
	}
	
	// TODO: turn off visibility logic in the Details Panel for now (MH-13360) as considered confusing to users; we may revisit this
	//if (PropertyNameToNodeTypeMapping.Contains(PropertyName))
	//{
	//	return StructureView->IsNodeOrChildSelected(PropertyNameToNodeTypeMapping[PropertyName], ArrayIndex);
	//}
	return true;
}

void FRigMapperDefinitionEditorToolkit::HandleGraphSelectionChanged(const TSet<UObject*>& Nodes)
{
	if (!StructureView || !DetailsView)
	{
		return;
	}

	StructureView->ClearSelection();
	DetailsView->SetObject(nullptr);
	SelectedGraphNode = nullptr;
	for (const UObject* ObjectNode : Nodes)
	{
		if (const URigMapperDefinitionEditorGraphNode* Node = Cast<URigMapperDefinitionEditorGraphNode>(ObjectNode))
		{
			StructureView->SelectNode(Node->NodeName, Node->GetNodeType(), true);
			if (Nodes.Num() == 1)
			{
				SelectedGraphNode = const_cast<URigMapperDefinitionEditorGraphNode*>(Node);
				DetailsView->SetObject(SelectedGraphNode.Get());
				DetailsView->OnFinishedChangingProperties().Clear();
				DetailsView->OnFinishedChangingProperties().AddRaw(
					this, &FRigMapperDefinitionEditorToolkit::HandleNodePropertyChanged);
			}
		}
		else if (const URigMapperCommentNode* CommentNode = Cast<URigMapperCommentNode>(ObjectNode))
		{
			if (Nodes.Num() == 1)
			{
				DetailsView->SetObject(const_cast<URigMapperCommentNode*>(CommentNode));
				DetailsView->OnFinishedChangingProperties().Clear();
				DetailsView->OnFinishedChangingProperties().AddRaw(
					this, &FRigMapperDefinitionEditorToolkit::HandleNodePropertyChanged);
			}
		}
	}

	if (GraphTab.IsValid())
	{
		FGlobalTabmanager::Get()->SetActiveTab(GraphTab);
	}

	FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::SetDirectly);
	if (GraphEditor)
	{
		GraphEditor->CaptureKeyboard();		
	}
}

void FRigMapperDefinitionEditorToolkit::HandleStructureSelectionChanged(ESelectInfo::Type SelectInfo, TArray<FString> SelectedInputs, TArray<FString> SelectedFeatures, TArray<FString> SelectedOutputs, TArray<FString> SelectedNullOutputs)
{
	if (GraphEditor && SelectInfo != ESelectInfo::Direct)
	{
		GraphEditor->SelectNodes(SelectedInputs, SelectedFeatures, SelectedOutputs, SelectedNullOutputs);
	};
}

void FRigMapperDefinitionEditorToolkit::HandleNodePropertyChanged(
	const FPropertyChangedEvent& PropertyChangedEvent)
{
	// Refresh the visual appearance of the selected node
	if (GraphEditor.IsValid() && SelectedGraphNode.IsValid())
	{
		// Get selected node and refresh its visual
		GraphEditor->RefreshGraphNode(SelectedGraphNode.Get());
		if (StructureView)
		{
			StructureView->RebuildTree();
		}
	}
}

void FRigMapperDefinitionEditorToolkit::PostUndo(bool bSuccess)
{
	if (bSuccess && GraphEditor.IsValid())
	{
		if (URigMapperDefinitionEditorGraph* Graph =
			Cast<URigMapperDefinitionEditorGraph>(GraphEditor->GetEditorGraphObject()))
		{
			Graph->RebuildNodeMaps();
			Graph->NotifyGraphChanged();
		}
		SelectedGraphNode = nullptr;
		if (DetailsView)
		{
			DetailsView->SetObject(nullptr);
		}
		if (StructureView) 
		{ 
			StructureView->RebuildTree(); 
		}
	}
}

void FRigMapperDefinitionEditorToolkit::PostRedo(bool bSuccess)
{
	PostUndo(bSuccess);
}
#undef LOCTEXT_NAMESPACE