// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "RigMapperDefinition.h"

#include "Toolkits/AssetEditorToolkit.h"
#include "EditorUndoClient.h"

#define UE_API RIGMAPPEREDITOR_API

enum class ERigMapperFeatureType : uint8;
class URigMapperDefinitionEditorGraphNode;
class SRigMapperDefinitionStructureView;
class SRigMapperDefinitionGraphEditor;

/**
 * The toolkit for the URigMapperDefinition asset editor
 */
class FRigMapperDefinitionEditorToolkit : public FAssetEditorToolkit, 
	public FSelfRegisteringEditorUndoClient
{
public:
	UE_API void Initialize(URigMapperDefinition* InDefinition, EToolkitMode::Type InMode, TSharedPtr<IToolkitHost> InToolkitHost);

	//~ FAssetEditorToolkit Interface
	UE_API virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;
	UE_API virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;
	UE_API virtual FName GetToolkitFName() const override;
	UE_API virtual FText GetBaseToolkitName() const override;
	UE_API virtual FString GetWorldCentricTabPrefix() const override;
	UE_API virtual FLinearColor GetWorldCentricTabColorScale() const override;
	UE_API virtual void SaveAsset_Execute() override;
	//~ FAssetEditorToolkit interface

	//~ FSelfRegisteringEditorUndoClient Interface
	UE_API void PostUndo(bool bSuccess) override;
	UE_API void PostRedo(bool bSuccess) override;
	//~ FSelfRegisteringEditorUndoClient Interface

	TSharedPtr<SRigMapperDefinitionGraphEditor> GetGraphEditorWidget() const
	{
		return GraphEditor;
	}

	class FDetailsViewCustomization : public IDetailCustomization
	{
	public:
		/** Makes a new instance of this detail layout class for a specific detail view requesting it */
		static TSharedRef<IDetailCustomization> MakeInstance();

		FDetailsViewCustomization()
		{}

		// IDetailCustomization interface
		virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	};

protected:
	void OnClose() override;
	
private:
	UE_API TSharedRef<SDockTab> SpawnGraphTab(const FSpawnTabArgs& SpawnTabArgs);
	UE_API TSharedRef<SDockTab> SpawnDetailsTab(const FSpawnTabArgs& SpawnTabArgs);
	UE_API TSharedRef<SDockTab> SpawnStructureTab(const FSpawnTabArgs& SpawnTabArgs);

	UE_API bool HandleIsPropertyVisible(const FPropertyAndParent& PropertyAndParent);
	UE_API void HandleNodePropertyChanged(const FPropertyChangedEvent& PropertyChangedEvent);
	UE_API void HandleGraphStructureUpdated();
	UE_API void HandleGraphSelectionChanged(const TSet<UObject*>& Nodes);
	
	UE_API void HandleStructureSelectionChanged(ESelectInfo::Type SelectInfo, TArray<FString> SelectedInputs, TArray<FString> SelectedFeatures, TArray<FString> SelectedOutputs, TArray<FString> SelectedNullOutputs);

private:
	static UE_API const FName DefinitionEditorGraphTabId;
	static UE_API const FName DefinitionEditorStructureTabId;
	static UE_API const FName DefinitionEditorDetailsTabId;

	static UE_API const TMap<FName, ERigMapperFeatureType> PropertyNameToNodeTypeMapping;
	
	TObjectPtr<URigMapperDefinition> Definition;

	TSharedPtr<SRigMapperDefinitionGraphEditor> GraphEditor;
	TSharedPtr<IDetailsView> DetailsView;
	TSharedPtr<SRigMapperDefinitionStructureView> StructureView;
	TSharedPtr<SDockTab> GraphTab;
	TWeakObjectPtr<URigMapperDefinitionEditorGraphNode> SelectedGraphNode;

};

#undef UE_API
