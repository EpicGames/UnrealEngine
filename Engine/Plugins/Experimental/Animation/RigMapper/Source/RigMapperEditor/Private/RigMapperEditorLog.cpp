// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigMapperEditorLog.h"

DEFINE_LOG_CATEGORY(LogRigMapperEditor);