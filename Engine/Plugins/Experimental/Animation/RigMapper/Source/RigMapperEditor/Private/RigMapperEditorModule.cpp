// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigMapperEditorModule.h"

#include "RigMapperDefinition.h"
#include "RigMapperDefinitionAssetTypeActions.h"
#include "RigMapperGraph/RigMapperDefinitionEditorGraphSchema.h"
#include "RigMapperLinkedDefinitionsAssetTypeActions.h"

#include "AssetToolsModule.h"
#include "ContentBrowserMenuContexts.h"
#include "DesktopPlatformModule.h"
#include "EdGraphUtilities.h"
#include "EditorDirectories.h"
#include "IDesktopPlatform.h"
#include "MessageLogModule.h"
#include "Brushes/SlateImageBrush.h"
#include "Framework/Application/SlateApplication.h"
#include "Modules/ModuleManager.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Toolkits/AssetEditorToolkitMenuContext.h"
#include "ToolMenus.h"
#include "Logging/MessageLog.h"
#include "RigMapperGraph/RigMapperDefinitionEditorGraph.h"
#include "RigMapperGraph/RigMapperDefinitionEditorGraphNode.h"
#include "RigMapperGraph/RigMapperGraphCommands.h"
#include "RigMapperGraph/SRigMapperDefinitionGraphEditor.h"
#include "RigMapperGraph/SRigMapperValidationBanner.h"
#include "RigMapperDefinitionEditorToolkit.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Paths.h"
#include "ScopedTransaction.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/AnimCurveMetadata.h"
#include "ControlRig.h"
#include "ControlRigBlueprintLegacy.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"

const FName FRigMapperEditorModule::AppIdentifier("RigMapperEditorApp");

const FName FRigMapperEditorModule::MessageLogIdentifier("RigMapperEditor");

#define LOCTEXT_NAMESPACE "RigMapperEditorModule"

void FRigMapperEditorModule::StartupModule()
{
	RegisterSlateStyle();
	
	if (FModuleManager::Get().IsModuleLoaded("AssetTools"))
	{
		// Register asset, editor & asset actions
		RigMapperDefinitionAssetTypeActions = MakeShared<FRigMapperDefinitionAssetTypeActions>();
		RigMapperLinkedDefinitionsAssetTypeActions = MakeShared<FRigMapperLinkedDefinitionsAssetTypeActions>();
		
		FAssetToolsModule::GetModule().Get().RegisterAssetTypeActions(RigMapperDefinitionAssetTypeActions.ToSharedRef());
		FAssetToolsModule::GetModule().Get().RegisterAssetTypeActions(RigMapperLinkedDefinitionsAssetTypeActions.ToSharedRef());
	}
	
	FToolMenuOwnerScoped OwnerScoped(this);

	RegisterRigMapperDefinitionContextMenuEntries();
	RegisterRigMapperDefinitionToolbarEntries();
	RegisterRigMapperLinkedDefinitionsContextMenuEntries();
	RegisterRigMapperLinkedDefinitionsToolbarEntries();

	FRigMapperGraphCommands::Register();

	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	MessageLogModule.RegisterLogListing(MessageLogIdentifier, LOCTEXT("RigMapperEditorMessageLog", "Rig Mapper"));

	RigMapperDefinitionGraphEditorNodeFactory = MakeShareable(new URigMapperDefinitionEditorGraphNode::NodeFactory());
	FEdGraphUtilities::RegisterVisualNodeFactory(RigMapperDefinitionGraphEditorNodeFactory);
}

void FRigMapperEditorModule::ShutdownModule()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*Style.Get());
	Style.Reset();
	
	if (FModuleManager::Get().IsModuleLoaded("AssetTools"))
	{
		FAssetToolsModule::GetModule().Get().UnregisterAssetTypeActions(RigMapperDefinitionAssetTypeActions.ToSharedRef());
	}

	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	MessageLogModule.UnregisterLogListing(MessageLogIdentifier);

	if (RigMapperDefinitionGraphEditorNodeFactory.IsValid())
	{
		FEdGraphUtilities::UnregisterVisualNodeFactory(RigMapperDefinitionGraphEditorNodeFactory);
	}

	FRigMapperGraphCommands::Unregister();
}

void FRigMapperEditorModule::RegisterSlateStyle()
{
	Style = MakeShareable(new FSlateStyleSet("RigMapperEditorStyle"));
	Style->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	Style->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));
	
	Style->Set("ClassIcon.RigMapperDefinition", new FSlateVectorImageBrush(
		Style->RootToContentDir("Starship/Common/Blueprint", TEXT(".svg")),
		FVector2D(16.0f, 16.0f)));
	Style->Set("ClassThumbnail.RigMapperDefinition", new FSlateVectorImageBrush(
		Style->RootToContentDir("Starship/Common/Blueprint", TEXT(".svg"))
		, FVector2D(64.0f, 64.0f)));

	Style->Set("ClassIcon.RigMapperLinkedDefinitions", new FSlateVectorImageBrush(
		Style->RootToContentDir("Starship/Common/Struct", TEXT(".svg")),
		FVector2D(16.0f, 16.0f)));
	Style->Set("ClassThumbnail.RigMapperLinkedDefinitions", new FSlateVectorImageBrush(
		Style->RootToContentDir("Starship/Common/Struct", TEXT(".svg"))
		, FVector2D(64.0f, 64.0f)));
	
	FSlateStyleRegistry::RegisterSlateStyle(*Style.Get());
}

void FRigMapperEditorModule::RegisterRigMapperDefinitionContextMenuEntries()
{
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu.RigMapperDefinition");

	const FToolMenuInsert SectionInsertLocation("GetAssetActions", EToolMenuInsertType::After);
	FToolMenuSection& Section = Menu->AddSection("RigMapperActions", LOCTEXT("RigMapperMenuSectionName", "Rig Mapper"), SectionInsertLocation);

	Section.AddMenuEntry(
		"RigMapperDefinition_LoadFromJson",
		LOCTEXT("RigMapperDefinitionContextMenu_LoadFromJson_Label", "Load from Json"),
		LOCTEXT("RigMapperDefinitionContextMenu_LoadFromJson_Tooltip", "Reload this definition from the given json file"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Toolbar.Import"),
		FToolMenuExecuteAction::CreateStatic(&FRigMapperEditorModule::LoadFromJson, false)
	);
	Section.AddMenuEntry(
		"RigMapperDefinition_ExportToJson",
		LOCTEXT("RigMapperDefinitionContextMenu_ExportToJson_Label", "Export to Json"),
		LOCTEXT("RigMapperDefinitionContextMenu_ExportToJson_Tooltip", "Export this definition as a json file"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Toolbar.Export"),
			FToolMenuExecuteAction::CreateStatic(&FRigMapperEditorModule::ExportToJson, false)
			);
			Section.AddMenuEntry(
				"RigMapperDefinition_Validate",
				LOCTEXT("RigMapperDefinitionContextMenu_Validate_Label", "Validate Definition"),
				LOCTEXT("RigMapperDefinitionContextMenu_Validate_Tooltip", "Check if the definition has any noticable issue"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Validate"),
				FToolMenuExecuteAction::CreateStatic(&FRigMapperEditorModule::ValidateDefinition, false)
			);
}

void FRigMapperEditorModule::RegisterRigMapperDefinitionToolbarEntries()
{
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("AssetEditor.RigMapperDefinitionEditor.ToolBar");
	FToolMenuSection& Section = Menu->FindOrAddSection("AssetEditorActions");

	Section.AddMenuEntry(
		"RigMapperDefinition_LoadFromJson",
		LOCTEXT("RigMapperDefinitionToolbar_LoadFromJson_Label", "Load"),
		LOCTEXT("RigMapperDefinitionToolbar_LoadFromJson_Tooltip", "Reload this definition from the given json file"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Toolbar.Import"),
		FToolMenuExecuteAction::CreateStatic(&FRigMapperEditorModule::LoadFromJson, true)
	);
	Section.AddMenuEntry(
		"RigMapperDefinition_ExportToJson",
		LOCTEXT("RigMapperDefinitionToolbar_ExportToJson_Label", "Export"),
		LOCTEXT("RigMapperDefinitionToolbar_ExportToJson_Tooltip", "Export this definition as a json file"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Toolbar.Export"),
		FToolMenuExecuteAction::CreateStatic(&FRigMapperEditorModule::ExportToJson, true)
	);
	Section.AddMenuEntry(
		"RigMapperDefinition_Validate",
		LOCTEXT("RigMapperDefinitionToolbar_Validate_Label", "Validate"),
		LOCTEXT("RigMapperDefinitionToolbar_Validate_Tooltip", "Check if the definition has any noticable issue"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Validate"),
		FToolMenuExecuteAction::CreateStatic(&FRigMapperEditorModule::ValidateDefinition, true)
	);
	Section.AddMenuEntry(
		"RigMapperDefinition_PopulateInputsFromMesh",
		LOCTEXT("RigMapperDefinitionToolbar_InputImport_Label", "Populate Inputs from Mesh"),
		LOCTEXT("RigMapperDefinitionToolbar_InputImport_Tooltip", "Pick a Skeletal Mesh and auto-create Input nodes from its curves and controls"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Plus"),
		FToolMenuExecuteAction::CreateStatic(&FRigMapperEditorModule::PopulateInputNodesFromMesh, true)
	);
	Section.AddMenuEntry(
		"RigMapperDefinition_PopulateOutputsFromMesh",
		LOCTEXT("RigMapperDefinitionToolbar_OutputImport_Label", "Populate Outputs from Mesh"),
		LOCTEXT("RigMapperDefinitionToolbar_OutputImport_Tooltip", "Pick a Skeletal Mesh and auto-create Output nodes from its curves and controls"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Plus"),
		FToolMenuExecuteAction::CreateStatic(&FRigMapperEditorModule::PopulateOutputNodesFromMesh, true)
	);
}


void FRigMapperEditorModule::RegisterRigMapperLinkedDefinitionsContextMenuEntries()
{
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu.RigMapperLinkedDefinitions");

	const FToolMenuInsert SectionInsertLocation("GetAssetActions", EToolMenuInsertType::After);
	FToolMenuSection& Section = Menu->AddSection("RigMapperActions", LOCTEXT("RigMapperActionsSectionName", "Rig Mapper"), SectionInsertLocation);

	Section.AddMenuEntry(
		"RigMapperLinkedDefinitions_Bake",
		LOCTEXT("RigMapperLinkedDefinitionsContextMenu_Bake_Label", "Bake Definitions"),
		LOCTEXT("RigMapperLinkedDefinitionsContextMenu_Bake_Tooltip", "The linked definitions will be baked to the output definition"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "AssetEditor.Apply"),
		FToolMenuExecuteAction::CreateStatic(&FRigMapperEditorModule::BakeDefinitions, false)
	);
	Section.AddMenuEntry(
		"RigMapperLinkedDefinitions_Validate",
		LOCTEXT("RigMapperLinkedDefinitionsContextMenu_Validate_Label", "Validate Linked Definitions"),
		LOCTEXT("RigMapperLinkedDefinitionsContextMenu_Validate_Tooltip", "Check if the definitions are valid and link together correctly"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Validate"),
		FToolMenuExecuteAction::CreateStatic(&FRigMapperEditorModule::ValidateLinkedDefinitions, false)
	);
}

void FRigMapperEditorModule::RegisterRigMapperLinkedDefinitionsToolbarEntries()
{
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("AssetEditor.RigMapperLinkedDefinitionsEditor.ToolBar");
	FToolMenuSection& Section = Menu->FindOrAddSection("AssetEditorActions");

	// todo: bake to new

	Section.AddEntry(FToolMenuEntry::InitToolBarButton(
		"RigMapperLinkedDefinitions_Bake",
		FToolMenuExecuteAction::CreateStatic(&FRigMapperEditorModule::BakeDefinitions, true),
		LOCTEXT("RigMapperLinkedDefinitionsToolbar_Bake_Label", "Bake"),
		LOCTEXT("RigMapperLinkedDefinitionsToolbar_Bake_Tooltip", "The linked definitions will be baked to the output definition"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "AssetEditor.Apply")
	));

	Section.AddEntry(FToolMenuEntry::InitToolBarButton(
		"RigMapperLinkedDefinitions_Validate",
		FToolMenuExecuteAction::CreateStatic(&FRigMapperEditorModule::ValidateLinkedDefinitions, true),
		LOCTEXT("RigMapperLinkedDefinitionsToolbar_Validate_Label", "Validate"),
		LOCTEXT("RigMapperLinkedDefinitionsToolbar_Validate_Tooltip", "Check if the definitions are valid and link together correctly"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Validate")
	));
}

bool FRigMapperEditorModule::GetUserPickedPath(FString& OutPath, bool bImport)
{
	TArray<FString> Filenames;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();

	bool bOpen;

	if (bImport)
	{
		FString LastRigMapperImportDirectory;
		if (!GConfig->GetString(TEXT("RigMapper"), TEXT("LastImportDirectory"), LastRigMapperImportDirectory, GEditorPerProjectIni))
		{
			LastRigMapperImportDirectory = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_IMPORT);
		}

		bOpen = DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			NSLOCTEXT("RigMapperDefinitionIO", "ImportFromJson", "Import Definition from Json...").ToString(), 
			LastRigMapperImportDirectory,
			TEXT(""), 
			TEXT("json (*.json)|*.json|"),
			EFileDialogFlags::None,
			Filenames
		);
	}
	else
	{
		constexpr bool bDirectory = true;

		FString LastRigMapperExportDirectory;
		if (!GConfig->GetString(TEXT("RigMapper"), TEXT("LastExportDirectory"), LastRigMapperExportDirectory, GEditorPerProjectIni))
		{
			LastRigMapperExportDirectory = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_EXPORT);
		}
		
		if (bDirectory)
		{
			return DesktopPlatform->OpenDirectoryDialog(
				FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
				NSLOCTEXT("RigMapperDefinitionIO", "ExportToJson", "Export Definition to Json...").ToString(), 
				LastRigMapperExportDirectory,
				OutPath
			);
		}
		bOpen = DesktopPlatform->SaveFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			NSLOCTEXT("RigMapperDefinitionIO", "ExportToJson", "Export Definition to Json...").ToString(), 
			LastRigMapperExportDirectory,
			TEXT(""), 
			TEXT("json (*.json)|*.json|"),
			EFileDialogFlags::None,
			Filenames
		);
	}
	
	if (bOpen && !Filenames.IsEmpty())
	{
		OutPath = Filenames[0];
	}

	return bOpen;
}

void FRigMapperEditorModule::LoadFromJson(const FToolMenuContext& ToolMenuContext, bool bFromAssetEditor)
{
	const TArray<URigMapperDefinition*> Definitions = GetDefinitionsFromContext(ToolMenuContext, bFromAssetEditor);
	
	FFilePath JsonFile;
	bool bSucceeded = false;
	FString LastPath;

	if (!Definitions.IsEmpty() && GetUserPickedPath(JsonFile.FilePath))
	{
		FScopedTransaction Transaction(LOCTEXT("LoadFromJson", "Load RigMapper Definitions from Json file"));

		for (URigMapperDefinition* Definition : Definitions)
		{
			Definition->Modify();
			if (!Definition->LoadFromJsonFile(JsonFile))
			{
				FMessageLog MessageLog(MessageLogIdentifier);

				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("ObjectName"), FText::FromString(Definition->GetName()));
	
				MessageLog.Error(FText::Format(LOCTEXT("RigMapperEditorActions_ImportFailed", "Failed to load definition \"{ObjectName}\" from Json. See output log for more details"), Arguments));
				MessageLog.Open(EMessageSeverity::Error);
				Transaction.Cancel();
				break;
			}
			else
			{
				bSucceeded = true;

				if (URigMapperDefinitionEditorGraph* Graph = Cast<URigMapperDefinitionEditorGraph>(Definition->EditorGraph))
				{
					Graph->Modify();
					Graph->RebuildGraph();
					Graph->LayoutNodes();
				}

				LastPath = JsonFile.FilePath;
			}
		}
	}

	if (bSucceeded)
	{
		// set the new import directory from the last successful save
		FString NewLastRigMapperImportDirectory = FPaths::GetPath(LastPath);
		GConfig->SetString(TEXT("RigMapper"), TEXT("LastImportDirectory"), *NewLastRigMapperImportDirectory, GEditorPerProjectIni);
		GConfig->Flush(false, GEditorPerProjectIni);
	}
}

void FRigMapperEditorModule::ExportToJson(const FToolMenuContext& ToolMenuContext, bool bFromAssetEditor)
{
	const TArray<URigMapperDefinition*> Definitions = GetDefinitionsFromContext(ToolMenuContext, bFromAssetEditor);
	
	FDirectoryPath Path;
	bool bSucceeded = false;
	FString LastPath;
	
	if (!Definitions.IsEmpty() && GetUserPickedPath(Path.Path, false))
	{
		for (URigMapperDefinition* Definition : Definitions)
		{
			FFilePath ActualFilePath;
			ActualFilePath.FilePath = FPaths::Combine(Path.Path, Definition->GetName() + ".json");

			if (!Definition->ExportAsJsonFile(ActualFilePath))
			{
				FMessageLog MessageLog(MessageLogIdentifier);

				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("ObjectName"), FText::FromString(Definition->GetName()));
	
				MessageLog.Error(FText::Format(LOCTEXT("RigMapperEditorActions_ExportFailed", "Failed to export definition \"{ObjectName}\". See output log for more details"), Arguments));
				MessageLog.Open(EMessageSeverity::Error);
			}
			else
			{
				bSucceeded = true;
				LastPath = ActualFilePath.FilePath;
			}
		}
	}

	if (bSucceeded)
	{
		// set the new export directory
		FString NewLastRigMapperExportDirectory = FPaths::GetPath(LastPath);
		GConfig->SetString(TEXT("RigMapper"), TEXT("LastExportDirectory"), *NewLastRigMapperExportDirectory, GEditorPerProjectIni);
		GConfig->Flush(false, GEditorPerProjectIni);
	}
}

void FRigMapperEditorModule::ValidateDefinition(const FToolMenuContext& ToolMenuContext, bool bFromAssetEditor)
{
	const TArray<URigMapperDefinition*> Definitions = GetDefinitionsFromContext(ToolMenuContext, bFromAssetEditor);

	for (URigMapperDefinition* Definition : Definitions)
	{
		FRigMapperValidationContext Context;
		const bool bValid = Definition->Validate(&Context);

		// Log to output log
		Definition->LogAll(Context);

		// Update the validation banner
		UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		IAssetEditorInstance* EditorInstance = AssetEditorSubsystem->FindEditorForAsset(Definition, false);
		FRigMapperDefinitionEditorToolkit* Toolkit = static_cast<FRigMapperDefinitionEditorToolkit*>(EditorInstance);
		if (Toolkit)
		{
			if (TSharedPtr<SRigMapperDefinitionGraphEditor> GraphEditor = Toolkit->GetGraphEditorWidget())
			{
				if (TSharedPtr<SRigMapperValidationBanner> Banner = GraphEditor->GetValidationBanner())
				{
					Banner->SetValidationResult(Context);
				}
			}
		}

		// Open MessageLog for errors, if any
		if (Context.HasErrors())
		{
			FMessageLog MessageLog(MessageLogIdentifier);
			for (const FRigMapperValidationMessage& Msg : Context.Messages)
			{
				if (Msg.Severity == ERigMapperValidationSeverity::Error)
				{
					MessageLog.Error(FText::FromString(Msg.Message));
				}
				else
				{
					MessageLog.Warning(FText::FromString(Msg.Message));
				}
			}
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("ObjectName"), FText::FromString(Definition->GetName()));
			MessageLog.Error(FText::Format(LOCTEXT("RigMapperEditorActions_ValidateFailed", "Failed to validate definition \"{ObjectName}\". See output log for more details"), Arguments));
			MessageLog.Open(EMessageSeverity::Error);
		}
	}
}

void FRigMapperEditorModule::BakeDefinitions(const FToolMenuContext& ToolMenuContext, bool bFromAssetEditor)
{
	TArray<URigMapperLinkedDefinitions*> Definitions = GetLinkedDefinitionsFromContext(ToolMenuContext, bFromAssetEditor);
	
	for (URigMapperLinkedDefinitions* LinkedDefinition : Definitions)
	{
		FRigMapperValidationContext Context;
		const bool bValid = LinkedDefinition->BakeDefinitions(&Context);

		// Log to output log
		LinkedDefinition->LogAll(Context);

		FNotificationInfo Info(FText::GetEmpty());
		Info.bFireAndForget = false;  // Persists until dismissed
		Info.FadeOutDuration = 0.0f;
		Info.ExpireDuration = 0.0f;

		if (Context.HasErrors())
		{
			Info.Text = FText::Format(LOCTEXT("LinkedDefBakingFailed", "Baking linked definitions failed: {0} error(s), {1} warning(s)"),
				FText::AsNumber(Context.GetErrorCount()), FText::AsNumber(Context.GetWarningCount()));
			Info.Image = FAppStyle::GetBrush("Icons.ErrorWithColor");
		}
		else if (Context.HasWarnings())
		{
			Info.Text = FText::Format(LOCTEXT("LinkedDefBakingWarnings", "Baking linked definitions passed with {0} warning(s)"),
				FText::AsNumber(Context.GetWarningCount()));
			Info.Image = FAppStyle::GetBrush("Icons.WarningWithColor");
		}
		else
		{
			Info.Text = LOCTEXT("LinkedDefBakingSuccess", "Baking linked definitions successful");
			Info.Image = FAppStyle::GetBrush("Icons.SuccessWithColor");
			Info.bFireAndForget = true;
			Info.ExpireDuration = 5.0f;  // Auto-dismiss success after 5s
		}

		TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
		if (Notification)
		{
			Notification->SetCompletionState(Context.HasErrors() ? SNotificationItem::CS_Fail :
				Context.HasWarnings() ? SNotificationItem::CS_None :
				SNotificationItem::CS_Success);
		}

		// Open MessageLog for errors, if any
		if (Context.HasErrors())
		{
			FMessageLog MessageLog(MessageLogIdentifier);
			for (const FRigMapperValidationMessage& Msg : Context.Messages)
			{
				if (Msg.Severity == ERigMapperValidationSeverity::Error)
				{
					MessageLog.Error(FText::FromString(Msg.Message));
				}
				else
				{
					MessageLog.Warning(FText::FromString(Msg.Message));
				}
			}
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("ObjectName"), FText::FromString(LinkedDefinition->GetName()));

			MessageLog.Error(FText::Format(LOCTEXT("RigMapperEditorActions_BakeFailed", "Failed to bake definitions from Linked Definition \"{ObjectName}\". See output log for more details"), Arguments));
			MessageLog.Open(EMessageSeverity::Error);
		}
	}
}

void FRigMapperEditorModule::ValidateLinkedDefinitions(const FToolMenuContext& ToolMenuContext, bool bFromAssetEditor)
{
	TArray<URigMapperLinkedDefinitions*> Definitions = GetLinkedDefinitionsFromContext(ToolMenuContext, bFromAssetEditor);

	for (URigMapperLinkedDefinitions* LinkedDefinition : Definitions)
	{
		FRigMapperValidationContext Context;
		const bool bValid = LinkedDefinition->Validate(&Context);

		// Log to output log
		LinkedDefinition->LogAll(Context);

		FNotificationInfo Info(FText::GetEmpty());
		Info.bFireAndForget = false;  // Persists until dismissed
		Info.FadeOutDuration = 0.0f;
		Info.ExpireDuration = 0.0f;

		if (Context.HasErrors())
		{
			Info.Text = FText::Format(LOCTEXT("LinkedDefValidationFailed", "Validation failed: {0} error(s), {1} warning(s)"),
				FText::AsNumber(Context.GetErrorCount()), FText::AsNumber(Context.GetWarningCount()));
			Info.Image = FAppStyle::GetBrush("Icons.ErrorWithColor");
		}
		else if (Context.HasWarnings())
		{
			Info.Text = FText::Format(LOCTEXT("LinkedDefValidationWarnings", "Validation passed with {0} warning(s)"),
				FText::AsNumber(Context.GetWarningCount()));
			Info.Image = FAppStyle::GetBrush("Icons.WarningWithColor");
		}
		else
		{
			Info.Text = LOCTEXT("LinkedDefValidationSuccess", "Validation successful");
			Info.Image = FAppStyle::GetBrush("Icons.SuccessWithColor");
			Info.bFireAndForget = true;
			Info.ExpireDuration = 5.0f;  // Auto-dismiss success after 5s
		}

		TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
		if (Notification)
		{
			Notification->SetCompletionState(Context.HasErrors() ? SNotificationItem::CS_Fail :
				Context.HasWarnings() ? SNotificationItem::CS_None :
				SNotificationItem::CS_Success);
		}

		// Open MessageLog for errors, if any
		if (Context.HasErrors())
		{
			FMessageLog MessageLog(MessageLogIdentifier);
			for (const FRigMapperValidationMessage& Msg : Context.Messages)
			{
				if (Msg.Severity == ERigMapperValidationSeverity::Error)
				{
					MessageLog.Error(FText::FromString(Msg.Message));
				}
				else
				{
					MessageLog.Warning(FText::FromString(Msg.Message));
				}
			}
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("ObjectName"), FText::FromString(LinkedDefinition->GetName()));

			MessageLog.Error(FText::Format(LOCTEXT("RigMapperEditorActions_ValidateLinkFailed", "Failed to validate definitions from Linked Definition \"{ObjectName}\". See output log for more details"), Arguments));
			MessageLog.Open(EMessageSeverity::Error);
		}
	}
}

static void OpenImportCurvesDialogue(TArray<FName>& OutCurveNames)
{
	TSharedPtr<SWindow> PickerWindow;
	USkeletalMesh* SelectedMesh = nullptr;
	bool bImportControls = false;
	bool bUserConfirmed = false;

	FContentBrowserModule& ContentBrowserModule =
		FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.SelectionMode = ESelectionMode::Single;
	AssetPickerConfig.Filter.ClassPaths.Add(USkeletalMesh::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.bRecursiveClasses = false;
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;

	// Just store the selection — don't close the window
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateLambda(
		[&SelectedMesh](const FAssetData& AssetData)
		{
			SelectedMesh = Cast<USkeletalMesh>(AssetData.GetAsset());
		});

	// Double-click confirms the selection (common UX pattern)
	AssetPickerConfig.OnAssetDoubleClicked = FOnAssetDoubleClicked::CreateLambda(
		[&SelectedMesh, &bUserConfirmed, &PickerWindow](const FAssetData& AssetData)
		{
			SelectedMesh = Cast<USkeletalMesh>(AssetData.GetAsset());
			if (SelectedMesh)
			{
				bUserConfirmed = true;
				PickerWindow->RequestDestroyWindow();
			}
		});

	TSharedRef<SCheckBox> ImportControlsCheckBox = SNew(SCheckBox)
		.IsChecked(ECheckBoxState::Unchecked)
		.OnCheckStateChanged_Lambda(
			[&bImportControls](ECheckBoxState NewState)
			{
				bImportControls = (NewState == ECheckBoxState::Checked);
			})
		[
			SNew(STextBlock)
				.Text(LOCTEXT("RigMapperEditor_IncludeControlRigCurves", "Include curves from attached Control Rig"))
				.ToolTipText(LOCTEXT("RigMapperEditor_IncludeControlRigCurvesTooltip",
					"If enabled, also imports float controls and curves from the mesh's default animating rig."))
		];

	PickerWindow = SNew(SWindow)
		.Title(LOCTEXT("RigMapperEditorActions_PickSourceMesh", "Select Skeletal Mesh"))
		.ClientSize(FVector2D(500, 650))
		.SupportsMinimize(false).SupportsMaximize(false)
		[
			SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("Menu.Background"))
				.Padding(8)
				[
					SNew(SVerticalBox)

						// Asset picker
						+ SVerticalBox::Slot()
						.FillHeight(1.0f)
						[
							ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
						]

						// Checkbox
						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0, 8)
						[
							ImportControlsCheckBox
						]

						// OK / Cancel buttons
						+ SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Right)
						[
							SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.AutoWidth()
								.Padding(0, 0, 4, 0)
								[
									SNew(SButton)
										.Text(LOCTEXT("RigMapperEditor_OK", "OK"))
										.IsEnabled_Lambda([&SelectedMesh]() { return SelectedMesh != nullptr; })
										.OnClicked_Lambda(
											[&bUserConfirmed, &PickerWindow]()
											{
												bUserConfirmed = true;
												PickerWindow->RequestDestroyWindow();
												return FReply::Handled();
											})
								]
							+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SButton)
										.Text(LOCTEXT("RigMapperEditor_Cancel", "Cancel"))
										.OnClicked_Lambda(
											[&PickerWindow]()
											{
												PickerWindow->RequestDestroyWindow();
												return FReply::Handled();
											})
								]
						]
				]
		];

	GEditor->EditorAddModalWindow(PickerWindow.ToSharedRef());

	if (!bUserConfirmed || !SelectedMesh)
	{
		return;
	}

	// Skeleton-level curves
	if (USkeleton* Skeleton = SelectedMesh->GetSkeleton())
	{
		if (UAnimCurveMetaData* SkeletonCurveData = Skeleton->GetAssetUserData<UAnimCurveMetaData>())
		{
			SkeletonCurveData->ForEachCurveMetaData(
				[&OutCurveNames](const FName& CurveName, const FCurveMetaData& MetaData)
				{
					OutCurveNames.Add(CurveName);
				});
		}
	}

	// Mesh-level curves
	if (UAnimCurveMetaData* MeshCurveData = SelectedMesh->GetAssetUserData<UAnimCurveMetaData>())
	{
		MeshCurveData->ForEachCurveMetaData(
			[&OutCurveNames](const FName& CurveName, const FCurveMetaData& MetaData)
			{
				OutCurveNames.Add(CurveName);
			});
	}

	// Control Rig curves (only if user opted in)
	if (bImportControls)
	{
		if (UObject* DefaultAnimatingRig = SelectedMesh->GetDefaultAnimatingRig().LoadSynchronous())
		{
			if (UControlRigBlueprint* ControlRigBlueprint = Cast<UControlRigBlueprint>(DefaultAnimatingRig))
			{
				URigHierarchy* Hierarchy = ControlRigBlueprint->GetHierarchy();

				for (FRigControlElement* Control : Hierarchy->GetControls())
				{
					if (Control->Settings.ControlType == ERigControlType::Float)
					{
						OutCurveNames.Add(Control->GetFName());
					}
				}

				for (FRigCurveElement* Curve : Hierarchy->GetCurves())
				{
					OutCurveNames.Add(Curve->GetFName());
				}
			}
		}
	}
}

void FRigMapperEditorModule::PopulateOutputNodesFromMesh(const FToolMenuContext& ToolMenuContext, bool bFromAssetEditor)
{
	TArray<FName> CurveNames;
	OpenImportCurvesDialogue(CurveNames);

	if (CurveNames.Num() > 0)
	{
		TArray<URigMapperDefinition*> Definitions = GetDefinitionsFromContext(ToolMenuContext, bFromAssetEditor);
		FScopedTransaction Transaction(LOCTEXT("PopulateOutputNodesFromMesh", "Import output nodes from Skeletal Mesh"));

		for (URigMapperDefinition* Definition : Definitions)
		{
			// Get the live graph from the definition
			URigMapperDefinitionEditorGraph* Graph = Cast<URigMapperDefinitionEditorGraph>(Definition->EditorGraph);
			Definition->Modify();
			if (!Graph)
			{
				// Create graph at this point if it does not exist.
				Graph = NewObject<URigMapperDefinitionEditorGraph>(Definition, NAME_None, RF_Transactional);
				Definition->EditorGraph = Graph;
				Graph->Initialize(Definition);
				Graph->RebuildGraph();
				Graph->LayoutNodes();
			}
			else if (!Graph->GetDefinition())
			{
				// Existing graph is loaded from context but not yet opened in editor. Initialize it.
				Graph->Initialize(Definition);
				Graph->RebuildNodeMaps();
				Graph->NotifyGraphChanged();
			}

			Graph->Modify();
			FVector2D LastInputPos = Graph->GetNextNodeSlotPosition(ERigMapperFeatureType::Output);
			for (const FName& CurveName : CurveNames)
			{
				const FString NameStr = CurveName.ToString();

				if (URigMapperDefinitionEditorGraphNode* NewNode = Graph->CreateGraphNode(ERigMapperFeatureType::Output, nullptr, FVector2f(LastInputPos), false, NameStr))
				{
					LastInputPos = Graph->GetNextNodeSlotPositionFromNode(NewNode);
				}
			}
		}
	}
}

void FRigMapperEditorModule::PopulateInputNodesFromMesh(const FToolMenuContext& ToolMenuContext, bool bFromAssetEditor)
{
	TArray<FName> CurveNames;
	OpenImportCurvesDialogue(CurveNames);

	if (CurveNames.Num() > 0)
	{
		TArray<URigMapperDefinition*> Definitions = GetDefinitionsFromContext(ToolMenuContext, bFromAssetEditor);
		FScopedTransaction Transaction(LOCTEXT("PopulateInputNodesFromMesh", "Import input nodes from Skeletal Mesh"));

		for (URigMapperDefinition* Definition : Definitions)
		{
			// Get the live graph from the definition
			URigMapperDefinitionEditorGraph* Graph = Cast<URigMapperDefinitionEditorGraph>(Definition->EditorGraph);
			Definition->Modify();
			if (!Graph)
			{ 
				// Create graph at this point if it does not exist.
				Graph = NewObject<URigMapperDefinitionEditorGraph>(Definition, NAME_None, RF_Transactional);
				Definition->EditorGraph = Graph;
				Graph->Initialize(Definition);
				Graph->RebuildGraph();
				Graph->LayoutNodes();
			}
			else if (!Graph->GetDefinition())
			{
				// Existing graph is loaded from context but not yet opened in editor. Initialize it.
				Graph->Initialize(Definition);
				Graph->RebuildNodeMaps();
				Graph->NotifyGraphChanged();
			}

			Graph->Modify();
			FVector2D LastInputPos = Graph->GetNextNodeSlotPosition(ERigMapperFeatureType::Input);
			for (const FName& CurveName : CurveNames)
			{
				const FString NameStr = CurveName.ToString();

				if (URigMapperDefinitionEditorGraphNode* NewNode = Graph->CreateGraphNode(ERigMapperFeatureType::Input, nullptr, FVector2f(LastInputPos), false, NameStr))
				{
					LastInputPos = Graph->GetNextNodeSlotPositionFromNode(NewNode);
				}
			}
		}
	}
}

TArray<URigMapperLinkedDefinitions*> FRigMapperEditorModule::GetLinkedDefinitionsFromContext(const FToolMenuContext& ToolMenuContext, bool bFromAssetEditor)
{
	TArray<URigMapperLinkedDefinitions*> Definitions;
	
	if (bFromAssetEditor)
	{
		const UAssetEditorToolkitMenuContext* ToolkitMenuContext = ToolMenuContext.FindContext<UAssetEditorToolkitMenuContext>();
	
		if (ToolkitMenuContext && ToolkitMenuContext->Toolkit.IsValid())
		{
			for (UObject* Object : ToolkitMenuContext->GetEditingObjects())
			{
				if (URigMapperLinkedDefinitions* Definition = Cast<URigMapperLinkedDefinitions>(Object))
				{
					Definitions.Add(Definition);
				}
			}
		}
		
	}
	else
	{
		if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(ToolMenuContext))
		{
			Definitions = Context->LoadSelectedObjects<URigMapperLinkedDefinitions>();
		}
	}
	return Definitions;
}

TArray<URigMapperDefinition*> FRigMapperEditorModule::GetDefinitionsFromContext(const FToolMenuContext& ToolMenuContext, bool bFromAssetEditor)
{
	TArray<URigMapperDefinition*> Definitions;
	
	if (bFromAssetEditor)
	{
		const UAssetEditorToolkitMenuContext* ToolkitMenuContext = ToolMenuContext.FindContext<UAssetEditorToolkitMenuContext>();
	
		if (ToolkitMenuContext && ToolkitMenuContext->Toolkit.IsValid())
		{
			for (UObject* Object : ToolkitMenuContext->GetEditingObjects())
			{
				if (URigMapperDefinition* Definition = Cast<URigMapperDefinition>(Object))
				{
					Definitions.Add(Definition);
				}
			}
		}
		
	}
	else
	{
		if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(ToolMenuContext))
		{
			Definitions = Context->LoadSelectedObjects<URigMapperDefinition>();
		}
	}
	return Definitions;
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FRigMapperEditorModule, RigMapperEditor)
