// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ToolMenuContext.h"
#include "Modules/ModuleInterface.h"

struct FGraphPanelNodeFactory;
class URigMapperDefinition;
class FRigMapperLinkedDefinitionsAssetTypeActions;
class URigMapperLinkedDefinitions;
class FSpawnTabArgs;
class SDockTab;
class FRigMapperDefinitionAssetTypeActions;
class FSlateStyleSet;

/**
 * FRigMapperEditorModule
 * The Editor Module for the Rig Mapper plugin
 */
class FRigMapperEditorModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

	static void RegisterRigMapperDefinitionContextMenuEntries();
	static void RegisterRigMapperDefinitionToolbarEntries();
	static void RegisterRigMapperLinkedDefinitionsContextMenuEntries();
	static void RegisterRigMapperLinkedDefinitionsToolbarEntries();
	
private:
	void RegisterSlateStyle();
	static bool GetUserPickedPath(FString& OutPath, bool bImport = true);
	static void LoadFromJson(const FToolMenuContext& ToolMenuContext, bool bFromAssetEditor);
	static void ExportToJson(const FToolMenuContext& ToolMenuContext, bool bFromAssetEditor);
	static void ValidateDefinition(const FToolMenuContext& ToolMenuContext, bool bFromAssetEditor);
	static void BakeDefinitions(const FToolMenuContext& ToolMenuContext, bool bFromAssetEditor);
	static void ValidateLinkedDefinitions(const FToolMenuContext& ToolMenuContext, bool bFromAssetEditor);
	static void PopulateInputNodesFromMesh(const FToolMenuContext& ToolMenuContext, bool bFromAssetEditor);
	static void PopulateOutputNodesFromMesh(const FToolMenuContext& ToolMenuContext, bool bFromAssetEditor);
	static TArray<URigMapperDefinition*> GetDefinitionsFromContext(const FToolMenuContext& ToolMenuContext, bool bFromAssetEditor);
	static TArray<URigMapperLinkedDefinitions*> GetLinkedDefinitionsFromContext(const FToolMenuContext& ToolMenuContext, bool bFromAssetEditor);
	
public:
	static const FName AppIdentifier;
	static const FName MessageLogIdentifier;
	
private:
	TSharedPtr<FSlateStyleSet> Style;
	
	TSharedPtr<FRigMapperDefinitionAssetTypeActions> RigMapperDefinitionAssetTypeActions;
	TSharedPtr<FRigMapperLinkedDefinitionsAssetTypeActions> RigMapperLinkedDefinitionsAssetTypeActions;
	TSharedPtr<FGraphPanelNodeFactory> RigMapperDefinitionGraphEditorNodeFactory;
};