// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigMapperEditorSubsystem.h"

#include "Engine/SkeletalMesh.h"
#include "Animation/AnimSequence.h"
#include "Misc/FrameNumber.h"
#include "AssetToolsModule.h"
#include "LevelSequence.h"
#include "SequencerUtilities.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/FileHelper.h"
#include "Sequencer/MovieSceneControlRigParameterSection.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "Tracks/MovieSceneSpawnTrack.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"
#include "Interfaces//ITargetPlatformManagerModule.h"
#include "RigMapperEditorLog.h"
#include "RigMapperProcessor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigMapperEditorSubsystem)

#define LOCTEXT_NAMESPACE "RigMapperEditorSubsystem"

UAnimSequence* CreateNewAnimSequence(const FString& BaseName, const FString& Path, USkeletalMesh* TargetMesh)
{
	if (!TargetMesh)
	{
		return nullptr;
	}

	FString UniqueAssetName = BaseName;
	
	FString UniquePackageName;
	const FString BasePackageName = FString(Path / UniqueAssetName);

	const FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().CreateUniqueAssetName(BasePackageName, "", /*out*/ UniquePackageName, /*out*/ UniqueAssetName);

	const bool bCreatePackage = !UniquePackageName.IsEmpty() && !UniqueAssetName.IsEmpty();
	UPackage* Package = bCreatePackage ? CreatePackage(*UniquePackageName) : GetTransientPackage();
	
	UAnimSequence* AnimSequence = FindObject<UAnimSequence>(Package, *UniqueAssetName);
	if (!AnimSequence)
	{
		AnimSequence = NewObject<UAnimSequence>(Package, *UniqueAssetName, bCreatePackage ? RF_Public | RF_Standalone : RF_NoFlags);
	}

	AnimSequence->SetSkeleton(TargetMesh->GetSkeleton());
	AnimSequence->SetPreviewMesh(TargetMesh);

	FAssetRegistryModule::AssetCreated(AnimSequence);
	
	return AnimSequence;
}

FGuid CreateSequencerSpawnable(const UMovieScene* MovieScene, UObject* ObjectToSpawn)
{
	if (!MovieScene || !ObjectToSpawn)
	{
		return FGuid();
	}

	UE::Sequencer::FCreateBindingParams CreateBindingParams;
	CreateBindingParams.bAllowCustomBinding = true;
	CreateBindingParams.bSpawnable = true;
	return FSequencerUtilities::CreateOrReplaceBinding(nullptr, MovieScene->GetTypedOuter<UMovieSceneSequence>(), ObjectToSpawn, CreateBindingParams);
}

UMovieSceneControlRigParameterSection* CreateControlRigSequence(const FString& PackageName, const FString& AssetName, const FFrameRate& FrameRate, const TSubclassOf<UControlRig>& ControlRigClass, USkeletalMesh* SkeletalMesh)
{
	if (!SkeletalMesh || !ControlRigClass)
	{
		UE_LOGF(LogRigMapperEditor, Error, "Could not create level sequence and control rig track to record, invalid skeletal mesh or control rig class provided")
		return nullptr;
	}

	FString UniquePackageName;
	FString UniqueAssetName = AssetName;
	const FString BasePackageName = FString(PackageName / UniqueAssetName);

	const FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().CreateUniqueAssetName(BasePackageName, "", /*out*/ UniquePackageName, /*out*/ UniqueAssetName);
	
	const bool bCreatePackage = !UniquePackageName.IsEmpty() && !UniqueAssetName.IsEmpty();
	UPackage* Package = bCreatePackage ? CreatePackage(*UniquePackageName) : GetTransientPackage();

	ULevelSequence* Sequence = FindObject<ULevelSequence>(Package, *UniqueAssetName);
	if (!Sequence)
	{
		Sequence = NewObject<ULevelSequence>(Package, *UniqueAssetName, bCreatePackage ? RF_Public | RF_Standalone : RF_NoFlags);
		Sequence->Initialize();
	}

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	check(MovieScene);

	MovieScene->Modify();

	MovieScene->SetDisplayRate(FrameRate);
	
	const FGuid NewGuid = Sequence->AllowsSpawnableObjects() ? CreateSequencerSpawnable(MovieScene, SkeletalMesh) : FGuid();
	if (!NewGuid.IsValid())
	{
		UE_LOGF(LogRigMapperEditor, Error, "Failed to create sequencer spawnable from the given skeletal mesh asset")
		return nullptr;
	}
	
	UMovieSceneControlRigParameterTrack* Track = Cast<UMovieSceneControlRigParameterTrack>(
		MovieScene->AddTrack(UMovieSceneControlRigParameterTrack::StaticClass(), NewGuid));
	check(Track);

	FString TrackName = ControlRigClass->GetName();
	TrackName.RemoveFromEnd(TEXT("_C"));

	Track->SetTrackName(*TrackName);
	Track->SetDisplayName(FText::FromString(TrackName));

	UControlRig* ControlRig = NewObject<UControlRig>(Track, ControlRigClass, *TrackName, RF_Transactional);

	ControlRig->Modify();
	ControlRig->Initialize();
	ControlRig->Evaluate_AnyThread();

	Track->Modify();
	
	UMovieSceneSection* NewSection = Track->CreateControlRigSection(0, ControlRig, true);
	check(NewSection);
	
	NewSection->Modify();
	NewSection->SetRange(TRange<FFrameNumber>());

	FAssetRegistryModule::AssetCreated(Sequence);
	Sequence->MarkPackageDirty();
	
	return Cast<UMovieSceneControlRigParameterSection>(NewSection);
}

ULevelSequence* URigMapperEditorSubsystem::GetSequenceFromSection(const UMovieSceneControlRigParameterSection* Section)
{
	if (Section)
	{
		return Cast<ULevelSequence>(Section->GetTypedOuter(ULevelSequence::StaticClass()));
	}
	return nullptr;
}

TArray<UMovieSceneControlRigParameterSection*> URigMapperEditorSubsystem::GetSectionsFromSequence(ULevelSequence* Sequence)
{
	TArray<UMovieSceneControlRigParameterSection*> Sections;
	
	const UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;
	if (MovieScene)
	{
		for (const FMovieSceneBinding& Binding : MovieScene->GetBindings())
		{
			for (const UMovieSceneTrack* Track : Binding.GetTracks())
			{
				if (Track->IsA(UMovieSceneControlRigParameterTrack::StaticClass()))
				{
					for (UMovieSceneSection* Section : Track->GetAllSections())
					{
						Sections.Add(Cast<UMovieSceneControlRigParameterSection>(Section));
					}
				}
			}
		}
	}
	return Sections;
}

FFrameRate URigMapperEditorSubsystem::GetAnimSequenceRate(const UAnimSequence* AnimSequence)
{
	return AnimSequence ? AnimSequence->GetTargetSamplingFrameRate(GetTargetPlatformManager()->GetRunningTargetPlatform()) : FFrameRate();
}

void URigMapperEditorSubsystem::SetAnimSequenceRate(UAnimSequence* AnimSequence, FFrameRate FrameRate, bool bSetImportProperties)
{
	if (AnimSequence && FrameRate.IsValid())
	{
		if (bSetImportProperties)
		{
			AnimSequence->ImportFileFramerate = FrameRate.AsDecimal();
			AnimSequence->ImportResampleFramerate = FMath::RoundToInt32(FrameRate.AsDecimal());
		}
		
		IAnimationDataController& Controller = AnimSequence->GetController();
		Controller.OpenBracket(LOCTEXT("SetSequenceFrameRateOpenBracket", "Set Sequence FrameRate"));
		Controller.InitializeModel();
		Controller.SetFrameRate(FrameRate, false);
		Controller.NotifyPopulated();
		Controller.CloseBracket();
	
		AnimSequence->MarkPackageDirty();
	}
}

bool URigMapperEditorSubsystem::ConvertCurveValuesToCsv(TArray<FString>& CurveNames, const TArray<FFrameTime>& FrameTimes, FFrameValues& CurveValuesPerFrame, const FFilePath& OutputFile, const TArray<URigMapperDefinition*>& Definitions, bool bOutputIntermediateCsvFiles)
{
	FRigMapperProcessor RigMapperProcessor(Definitions);
	if (!RigMapperProcessor.IsValid())
	{
		return false;
	}

	// todo: bOutputIntermediateCsvFiles
	// const FString BaseFilePath = FPaths::GetBaseFilename(OutputFile.FilePath, false);;
	TArray<FName> CurveNamesConverted;
	CurveNamesConverted.Reserve(CurveNames.Num());
	for (const FString& Str : CurveNames)
	{
		CurveNamesConverted.Add(FName(*Str));
	}

	FFrameValues InputCurveValuesPerFrame = CurveValuesPerFrame; // take a copy as call to EvaluateFrames resets the output before evaluating the input
	RigMapperProcessor.EvaluateFrames(CurveNamesConverted, InputCurveValuesPerFrame, CurveNamesConverted, CurveValuesPerFrame);

	CurveNames.Empty();
	CurveNames.Reserve(CurveNamesConverted.Num()); 
	Algo::Transform(CurveNamesConverted, CurveNames, [](const FName& Name)
		{
			return Name.ToString();
		});

	WriteCurveValuesToCsv(OutputFile, CurveNames, FrameTimes, CurveValuesPerFrame);
	
	return true;
}

bool URigMapperEditorSubsystem::ConvertCsv(const FFilePath& InputFile, const FFilePath& OutputFile, const TArray<URigMapperDefinition*>& Definitions, bool bOutputIntermediateCsvFiles)
{
	TArray<FString> CurveNames;
	TArray<FFrameTime> FrameTimes;
		FFrameValues CurveValuesPerFrame;
	
	if (!LoadCurveValuesFromCsv(InputFile, CurveNames, FrameTimes, CurveValuesPerFrame))
	{
		return false;
	}

	return ConvertCurveValuesToCsv(CurveNames, FrameTimes, CurveValuesPerFrame, OutputFile, Definitions, bOutputIntermediateCsvFiles);
}

bool URigMapperEditorSubsystem::ConvertCurveValuesToAnimSequence(TArray<FString>& CurveNames, const TArray<FFrameTime>& FrameTimes, FFrameValues& CurveValuesPerFrame, UAnimSequence* Target, const TArray<URigMapperDefinition*>& Definitions)
{
	TArray<FName> CurveNamesConverted;
	CurveNamesConverted.Reserve(CurveNames.Num());
	for (const FString& Str : CurveNames)
	{
		CurveNamesConverted.Add(FName(*Str));
	}


	FRigMapperProcessor RigMapperProcessor(Definitions);
	if (!RigMapperProcessor.IsValid())
	{
		UE_LOGF(LogRigMapperEditor, Error, "Rig Mapper Processor is invalid")
		return false;
	}
	FFrameValues ConvertedCurveValuesPerFrame;
	if (!RigMapperProcessor.EvaluateFrames(CurveNamesConverted, CurveValuesPerFrame, CurveNamesConverted, ConvertedCurveValuesPerFrame))
	{
		UE_LOGF(LogRigMapperEditor, Error, "Rig Mapper Processor failed to evaluate frames")
		return false;
	}

	CurveNames.Empty();
	CurveNames.Reserve(CurveNamesConverted.Num());
	Algo::Transform(CurveNamesConverted, CurveNames, [](const FName& Name)
		{
			return Name.ToString();
		});


	return AddCurveValuesToAnimSequence(Target, CurveNames, FrameTimes, ConvertedCurveValuesPerFrame);
}

bool URigMapperEditorSubsystem::AddCurveValuesToAnimSequence(UAnimSequence* Target, const TArray<FString>& CurveNames, const TArray<FFrameTime>& FrameTimes, const FFrameValues& CurveValuesPerFrame)
{	
	if (CurveNames.IsEmpty())
	{
		UE_LOGF(LogRigMapperEditor, Error, "Failed to add empty curves list")
		return false;
	}
	if (FrameTimes.IsEmpty())
	{
		UE_LOGF(LogRigMapperEditor, Error, "Failed to add curves with no frame times")
		return false;
	}
	if (CurveValuesPerFrame.Num() != FrameTimes.Num())
	{
		UE_LOGF(LogRigMapperEditor, Error, "Failed to add curves: number of frames does not match expected number (%d vs %d)", CurveValuesPerFrame.Num(), FrameTimes.Num())
		return false;
	}
	
	const FFrameRate FrameRate = GetAnimSequenceRate(Target);
	if (!FrameRate.IsValid())
	{
		UE_LOGF(LogRigMapperEditor, Error, "Invalid rate %f", FrameRate.AsDecimal())
		return false;
	}
	
	IAnimationDataController& Controller = Target->GetController();
	Controller.OpenBracket(LOCTEXT("PopulateNewAnimationOpenBracket", "Populate New Anim"));
	Controller.InitializeModel();
	Controller.RemoveAllAttributes(false);
	Controller.RemoveAllBoneTracks(false);
	Controller.RemoveAllCurvesOfType(ERawCurveTrackTypes::RCT_Float, false);
	Controller.RemoveAllCurvesOfType(ERawCurveTrackTypes::RCT_Transform, false);
	Controller.RemoveAllCurvesOfType(ERawCurveTrackTypes::RCT_Vector, false);
	Controller.SetNumberOfFrames(FrameTimes.Last().CeilToFrame().Value - FrameTimes[0].FloorToFrame().Value + 1, false);

	// AnimSequence should start on frame 0
	const float TimeOffset = FrameRate.AsSeconds(FrameTimes[0]);
	
	for (int32 CurveIndex = 0; CurveIndex < CurveNames.Num(); CurveIndex++)
	{
		FAnimationCurveIdentifier CurveId = UAnimationCurveIdentifierExtensions::GetCurveIdentifier(Target->GetSkeleton(), *CurveNames[CurveIndex], ERawCurveTrackTypes::RCT_Float);
		// todo: remap curve name from outputs

		if (CurveId.IsValid())
		{
			Controller.AddCurve(CurveId, 4, false);

			for (int32 FrameIndex = 0; FrameIndex < FrameTimes.Num(); FrameIndex++)
			{
				const float Time = FrameRate.AsSeconds(FrameTimes[FrameIndex]) - TimeOffset;
				const TOptional<float> Value = CurveValuesPerFrame[FrameIndex][CurveIndex];

				if (Value.IsSet())
				{
					Controller.SetCurveKey(CurveId, FRichCurveKey(Time, Value.GetValue()), false);
				}
			}
		}
	}
	
	Controller.NotifyPopulated();
	Controller.CloseBracket();	
	
	Target->MarkPackageDirty();

	return true;
}


bool URigMapperEditorSubsystem::LoadCurveValuesFromAnimSequence(const UAnimSequence* InSource, const USkeleton* InSourceSkeleton, TArray<FName>& OutCurveNames, TArray<FFrameTime>& OutFrameTimes, FFrameValues& OutCurveValuesPerFrame,
	TArray<int32>& OutCurveFlags, TArray<FLinearColor>& OutCurveColors)
{
	TArray<FString> CurveNames;
	if (LoadCurveValuesFromAnimSequence(InSource, CurveNames, OutFrameTimes, OutCurveValuesPerFrame))
	{
		OutCurveNames.Reserve(CurveNames.Num());
		OutCurveFlags.Reserve(CurveNames.Num());
		OutCurveColors.Reserve(CurveNames.Num());

		const IAnimationDataModel* SourceDataModel = InSource->GetDataModel();

		for (const FString& CurveName : CurveNames)
		{
			const FName CurCurveName(*CurveName);
			const FAnimationCurveIdentifier CurCurveId = UAnimationCurveIdentifierExtensions::FindCurveIdentifier(InSourceSkeleton, CurCurveName, ERawCurveTrackTypes::RCT_Float);

			const FFloatCurve* SourceCurve = SourceDataModel->FindFloatCurve(CurCurveId);
			if (SourceCurve)
			{
				OutCurveFlags.Add(SourceCurve->GetCurveTypeFlags());
				OutCurveColors.Add(SourceCurve->Color);
			}
			else
			{
				return false;
			}

			OutCurveNames.Add(CurCurveName);
		}
	}
	else
	{
		return false;
	}

	return true;
}


bool URigMapperEditorSubsystem::LoadCurveValuesFromAnimSequence(const UAnimSequence* Source, TArray<FString>& CurveNames, TArray<FFrameTime>& FrameTimes, FFrameValues& CurveValuesPerFrame)
{
	FFramePoses Poses;
	
	const TArray<FFloatCurve>& FloatCurves = Source->GetDataModel()->GetFloatCurves();

	if (!FloatCurves.IsEmpty())
	{
		FrameTimes.Reserve(FloatCurves[0].FloatCurve.GetNumKeys());

		for (const FFloatCurve& Curve : FloatCurves)
		{
			const FString CurveName = Curve.GetName().ToString();
			// todo: remap name from inputs

			if (!Curve.FloatCurve.Keys.IsEmpty())
			{
				CurveNames.AddUnique(CurveName);
				
				for (const FRichCurveKey& Key : Curve.FloatCurve.Keys)
				{
					const FFrameTime FrameNumber = GetAnimSequenceRate(Source).AsFrameTime(Key.Time);
					int32 PoseIndex = FrameTimes.Find(FrameNumber);

					if (PoseIndex == INDEX_NONE)
					{
						PoseIndex = FrameTimes.Add(FrameNumber);
						Poses.AddDefaulted_GetRef().Reserve(FloatCurves.Num());
					}

					Poses[PoseIndex].Add(CurveName, Key.Value);
				}
			}
		}
	}

	return BakeSparseKeys(Poses, CurveNames, FrameTimes, CurveValuesPerFrame);
}

bool URigMapperEditorSubsystem::ConvertAnimSequence(const UAnimSequence* Source, UAnimSequence* Target, const TArray<URigMapperDefinition*>& Definitions)
{
	TArray<FString> CurveNames;
	TArray<FFrameTime> FrameTimes;
	FFrameValues CurveValuesPerFrame;

	UE_LOGF(LogRigMapperEditor, Log, "Loading curves from anim sequence")
	if (!LoadCurveValuesFromAnimSequence(Source, CurveNames, FrameTimes, CurveValuesPerFrame))
	{
		return false;
	}
	UE_LOGF(LogRigMapperEditor, Log, "Converting curves to anim sequence: %d named curves, %d frame times, %d frame curves", CurveNames.Num(), FrameTimes.Num(), CurveValuesPerFrame.Num())
	
	return ConvertCurveValuesToAnimSequence(CurveNames, FrameTimes, CurveValuesPerFrame, Target, Definitions);
}

UAnimSequence* URigMapperEditorSubsystem::ConvertAnimSequenceNew(const UAnimSequence* Source, USkeletalMesh* TargetMesh, const TArray<URigMapperDefinition*>& Definitions, const FDirectoryPath& NewAssetPath, const FName NewAssetName)
{
	UAnimSequence* NewSequence = CreateNewAnimSequence(NewAssetName.ToString(), NewAssetPath.Path, TargetMesh);

	if (NewSequence)
	{
		NewSequence->ImportFileFramerate = Source->ImportFileFramerate;
		NewSequence->ImportResampleFramerate = Source->ImportResampleFramerate;
		SetAnimSequenceRate(NewSequence, GetAnimSequenceRate(Source), false);

		if (!ConvertAnimSequence(Source, NewSequence, Definitions))
		{
			UE_LOGF(LogRigMapperEditor, Error, "Failed to convert anim sequence using selected RigMapper Definitions");
			return nullptr;
		}

		NewSequence->PostLoad();
	}
	else
	{
		UE_LOGF(LogRigMapperEditor, Error, "Failed to create anim sequence %ls in folder %ls", *NewAssetName.ToString(), *NewAssetPath.Path);
	}
	return NewSequence;
}

bool URigMapperEditorSubsystem::ConvertCsvToAnimSequence(const FFilePath& InputFile, UAnimSequence* Target, const TArray<URigMapperDefinition*>& Definitions)
{
	TArray<FString> CurveNames;
	TArray<FFrameTime> FrameTimes;
	FFrameValues CurveValuesPerFrame;
	
	if (!LoadCurveValuesFromCsv(InputFile, CurveNames, FrameTimes, CurveValuesPerFrame))
	{
		return false;
	}

	return ConvertCurveValuesToAnimSequence(CurveNames, FrameTimes, CurveValuesPerFrame, Target, Definitions);
}

UAnimSequence* URigMapperEditorSubsystem::ConvertCsvToAnimSequenceNew(const FFilePath& InputFile, USkeletalMesh* TargetMesh, const TArray<URigMapperDefinition*>& Definitions, const FFrameRate& FrameRate, const FDirectoryPath& NewAssetPath, const FName NewAssetName)
{
	UAnimSequence* NewSequence = CreateNewAnimSequence(NewAssetName.ToString(), NewAssetPath.Path, TargetMesh);
	
	SetAnimSequenceRate(NewSequence, FrameRate, true);

	if (!ConvertCsvToAnimSequence(InputFile, NewSequence, Definitions))
	{
		return nullptr;
	}
	NewSequence->PostLoad();
	return NewSequence;
}

bool URigMapperEditorSubsystem::ConvertAnimSequenceToCsv(const UAnimSequence* Source, const FFilePath& OutputFile, const TArray<URigMapperDefinition*>& Definitions, bool bOutputIntermediateCsvFiles)
{
	TArray<FString> CurveNames;
	TArray<FFrameTime> FrameTimes;
	FFrameValues CurveValuesPerFrame;
	
	if (!LoadCurveValuesFromAnimSequence(Source, CurveNames, FrameTimes, CurveValuesPerFrame))
	{
		return false;
	}

	return ConvertCurveValuesToCsv(CurveNames, FrameTimes, CurveValuesPerFrame, OutputFile, Definitions, bOutputIntermediateCsvFiles);
}

bool URigMapperEditorSubsystem::ConvertAnimSequenceToControlRigSection(const UAnimSequence* Source, UMovieSceneControlRigParameterSection* Target, const TArray<URigMapperDefinition*>& Definitions)
{
	TArray<FString> CurveNames;
	TArray<FFrameTime> FrameTimes;
	FFrameValues CurveValuesPerFrame;
	
	if (!LoadCurveValuesFromAnimSequence(Source, CurveNames, FrameTimes, CurveValuesPerFrame))
	{
		return false;
	}

	return ConvertCurveValuesToControlRigSection(CurveNames, FrameTimes, CurveValuesPerFrame, Target, Definitions);
}

UMovieSceneControlRigParameterSection* URigMapperEditorSubsystem::ConvertAnimSequenceToControlRigSectionNew(const UAnimSequence* Source, USkeletalMesh* TargetMesh, const TArray<URigMapperDefinition*>& Definitions, const TSubclassOf<UControlRig>& ControlRigClass, const FDirectoryPath& NewAssetPath, const FName NewAssetName)
{
	UMovieSceneControlRigParameterSection* NewSection = CreateControlRigSequence(NewAssetPath.Path, NewAssetName.ToString(), Source->GetTypedOuter<UMovieScene>()->GetDisplayRate(), ControlRigClass, TargetMesh);
	
	TArray<FString> CurveNames;
	TArray<FFrameTime> FrameTimes;
	FFrameValues CurveValuesPerFrame;
	
	if (!LoadCurveValuesFromAnimSequence(Source, CurveNames, FrameTimes, CurveValuesPerFrame))
	{
		return nullptr;
	}
	
	if (ConvertCurveValuesToControlRigSection(CurveNames, FrameTimes, CurveValuesPerFrame, NewSection, Definitions))
	{
		return NewSection;
	}
	return nullptr;
}

bool URigMapperEditorSubsystem::ConvertCurveValuesToControlRigSection(TArray<FString>& CurveNames, const TArray<FFrameTime>& FrameTimes, FFrameValues& CurveValuesPerFrame, UMovieSceneControlRigParameterSection* Target, const TArray<URigMapperDefinition*>& Definitions)
{
	TArray<FName> CurveNamesConverted;
	CurveNamesConverted.Reserve(CurveNames.Num());
	for (const FString& Str : CurveNames)
	{
		CurveNamesConverted.Add(FName(*Str));
	}

	FRigMapperProcessor RigMapperProcessor(Definitions);
	if (!RigMapperProcessor.IsValid())
	{
		return false;
	}

	FFrameValues InputCurveValuesPerFrame = CurveValuesPerFrame; // take a copy as call to EvaluateFrames resets the output before evaluating the input
	RigMapperProcessor.EvaluateFrames(CurveNamesConverted, InputCurveValuesPerFrame, CurveNamesConverted, CurveValuesPerFrame);


	CurveNames.Empty();
	CurveNames.Reserve(CurveNamesConverted.Num());
	Algo::Transform(CurveNamesConverted, CurveNames, [](const FName& Name)
		{
			return Name.ToString();
		});

	return AddCurveValuesToControlRigSection(Target, CurveNames, FrameTimes, CurveValuesPerFrame);
}

int32 GetChannelIndexFromCurveName(const FString& CurveName, const TArrayView<const FMovieSceneChannelMetaData> MetaData)
{
	TArray<FString> Parts;
	CurveName.ParseIntoArray(Parts, TEXT("."));
	
	const FString ControlName = Parts[0];

	// todo: better mapping for custom attrs
	// .tx/ry/sz -> .Location.X/Rotation.Y/Scale.Z
	const FString ChannelTmName = ControlName + Parts.Last().ToUpper().Replace(
		TEXT("T"), TEXT(".Location.")).Replace(
			TEXT("R"), TEXT(".Rotation.")).Replace(
				TEXT("S"), TEXT(".Scale."));
	const FString Channel2dName = ChannelTmName.Replace(TEXT(".Location"), TEXT(""));

	int32 ChannelIndex = INDEX_NONE;
	for (int32 MetaDataIndex = 0; MetaDataIndex < MetaData.Num(); MetaDataIndex++)
	{
		const FString ChannelName = MetaData[MetaDataIndex].Name.ToString();
		
		if (ChannelName == ControlName || ChannelName == ChannelTmName || ChannelName == Channel2dName)
		{
			ChannelIndex = MetaDataIndex;
			break;
		}
	}
	
	return ChannelIndex;
}

bool URigMapperEditorSubsystem::AddCurveValuesToControlRigSection(UMovieSceneControlRigParameterSection* Target, const TArray<FString>& CurveNames, const TArray<FFrameTime>& FrameTimes, const FFrameValues& CurveValuesPerFrame)
{
	const UMovieScene* MovieScene = Target->GetTypedOuter<UMovieScene>();

	check(MovieScene);
	
	if (CurveNames.IsEmpty() || FrameTimes.IsEmpty() || CurveValuesPerFrame.IsEmpty())
	{
		return false;
	}

	Target->Modify();
	// Target->ClearAllParameters();
	// Target->GetControlRig()->RequestInit();
	//Target->GetControlRig()->Initialize();
	UControlRig* ControlRig = Target->GetControlRig(); // NewObject<UControlRig>(Target->GetOuter(), ->GetClass(), *Target->GetControlRig()->GetName(), RF_Transactional);
	ControlRig->Modify();
	ControlRig->Initialize();
	ControlRig->RequestInit();
	ControlRig->Execute(FRigUnit_PrepareForExecution::EventName);
	Target->RecreateWithThisControlRig(ControlRig, true);
	// Target->ReconstructChannelProxy();
	// Target->CacheChannelProxy();

	if (const FMovieSceneChannelEntry* Entry = Target->GetChannelProxy().FindEntry(FMovieSceneFloatChannel::StaticStruct()->GetFName()))
	{
		const TArrayView<FMovieSceneChannel* const> Channels = Entry->GetChannels();
		if (Channels.IsEmpty())
		{
			return false;
		}
		const TArrayView<const FMovieSceneChannelMetaData> MetaData = Entry->GetMetaData();
		
		for (int32 CurveIndex = 0; CurveIndex < CurveNames.Num(); CurveIndex++)
		{
			const int32 ChannelIndex = GetChannelIndexFromCurveName(CurveNames[CurveIndex], MetaData);
			// todo: remap from outputs

			if (Channels.IsValidIndex(ChannelIndex)) // todo: else log
			{
				FMovieSceneFloatChannel* Channel = static_cast<FMovieSceneFloatChannel*>(Channels[ChannelIndex]);

				// todo: do we want to clear the section?
				if (Channel->GetNumKeys() > 0)
				{
					Channel->DeleteKeysFrom(Channel->GetTimes()[0] - 1, false);
				}
				
				for (int32 FrameIndex = 0; FrameIndex < FrameTimes.Num(); FrameIndex++)
				{
					const TOptional<float> CurveValue = CurveValuesPerFrame[FrameIndex][CurveIndex];

					if (CurveValue.IsSet())
					{
						const FFrameNumber FrameNumber = MovieScene->GetTickResolution().AsFrameNumber(MovieScene->GetDisplayRate().AsSeconds(FrameTimes[FrameIndex]));
						Channel->AddCubicKey(FrameNumber, CurveValuesPerFrame[FrameIndex][CurveIndex].GetValue());
					}
				}
			}
		}
	}
	return true;
}

FString ChannelNameToCurveName(const FString& ChannelName, const TArray<FString>& CurveNames)
{
	if (CurveNames.Contains(ChannelName))
	{
		return ChannelName;
	}
	
	TArray<FString> Parts;
	ChannelName.ParseIntoArray(Parts, TEXT("."));
	
	const FString ControlName = Parts[0];
	FString ChannelToCurve;

	if (Parts.Last() == ControlName)
	{
		// todo: default
		ChannelToCurve = ControlName + ".ty";
	}
	else
	{
		if (Parts.Num() > 1)
		{
			ChannelToCurve = ControlName + Parts[1].Replace(
				TEXT("Location"), TEXT(".t")).Replace(
					TEXT("Rotation"), TEXT(".r")).Replace(
						TEXT("Scale"), TEXT(".s")).Replace(
							TEXT("X"), TEXT(".t")).Replace(
								TEXT("Y"), TEXT(".t"));
			ChannelToCurve += Parts.Last().ToLower();
		}
		// todo: any other remapping here?
	}
	
	if (CurveNames.Contains(ChannelToCurve))
	{
		return ChannelToCurve;
	}

	return ""; // todo: warn
}

bool URigMapperEditorSubsystem::LoadCurveValuesFromControlRigSection(const UMovieSceneControlRigParameterSection* Source, TArray<FString>& CurveNames, TArray<FFrameTime>& FrameTimes, FFrameValues& CurveValuesPerFrame, const TArray<FString>& InputNames)
{
	FFramePoses Poses;
	
	const UMovieScene* MovieScene = Source->GetTypedOuter<UMovieScene>();
	check(MovieScene);
	
	// todo: non float channels
	if (const FMovieSceneChannelEntry* Entry = Source->GetChannelProxy().FindEntry(FMovieSceneFloatChannel::StaticStruct()->GetFName()))
	{
		const TArrayView<FMovieSceneChannel* const> Channels = Entry->GetChannels();
		if (!Channels.IsEmpty())
		{
			const TArrayView<const FMovieSceneChannelMetaData> MetaData = Entry->GetMetaData();
			Poses.Reserve(Channels[0]->GetNumKeys());
		
			for (int32 ChannelIndex = 0; ChannelIndex < Channels.Num(); ChannelIndex++)
			{
				const FString ChannelName = MetaData[ChannelIndex].Name.ToString();
				const FString CurveName = ChannelNameToCurveName(ChannelName, InputNames);
				// todo: remap name from inputs

				if (!CurveName.IsEmpty())
				{
					CurveNames.AddUnique(CurveName);
				
					const FMovieSceneFloatChannel* Channel = static_cast<FMovieSceneFloatChannel*>(Channels[ChannelIndex]);
					TArrayView<const FFrameNumber> KeyTimes = Channel->GetTimes();
					TArrayView<const FMovieSceneFloatValue> KeyValues = Channel->GetValues();

					for (int32 KeyIndex = 0; KeyIndex < KeyTimes.Num() && KeyIndex < KeyValues.Num(); KeyIndex++)
					{
						const FFrameTime FrameNumber = MovieScene->GetDisplayRate().AsFrameTime(MovieScene->GetTickResolution().AsSeconds(KeyTimes[KeyIndex]));
						int32 PoseIndex = FrameTimes.Find(FrameNumber);
				
						if (PoseIndex == INDEX_NONE)
						{
							PoseIndex = FrameTimes.Add(FrameNumber);
							Poses.AddDefaulted_GetRef().Reserve(Channels.Num());
						}
				
						Poses[PoseIndex].Add(CurveName, KeyValues[KeyIndex].Value);	
					}
				}
			}
		}
	}
	
	return BakeSparseKeys(Poses, CurveNames, FrameTimes, CurveValuesPerFrame);
}

bool URigMapperEditorSubsystem::ConvertControlRigSection(const UMovieSceneControlRigParameterSection* Source, UMovieSceneControlRigParameterSection* Target, const TArray<URigMapperDefinition*>& Definitions)
{
	TArray<FString> CurveNames;
	TArray<FFrameTime> FrameTimes;
	FFrameValues CurveValuesPerFrame;

	if (Definitions.IsEmpty())
	{
		return false;
	}
	if (!LoadCurveValuesFromControlRigSection(Source, CurveNames, FrameTimes, CurveValuesPerFrame, Definitions[0]->Inputs))
	{
		return false;
	}
	
	return ConvertCurveValuesToControlRigSection(CurveNames, FrameTimes, CurveValuesPerFrame, Target, Definitions);
}

UMovieSceneControlRigParameterSection* URigMapperEditorSubsystem::ConvertControlRigSectionNew(const UMovieSceneControlRigParameterSection* Source, USkeletalMesh* TargetMesh, const TArray<URigMapperDefinition*>& Definitions, const TSubclassOf<UControlRig>& ControlRigClass, const FDirectoryPath& NewAssetPath, const FName NewAssetName)
{
	UMovieSceneControlRigParameterSection* NewSection = CreateControlRigSequence(NewAssetPath.Path, NewAssetName.ToString(), Source->GetTypedOuter<UMovieScene>()->GetDisplayRate(), ControlRigClass, TargetMesh);
	
	if (ConvertControlRigSection(Source, NewSection, Definitions))
	{
		return NewSection;
	}
	return nullptr;
}

bool URigMapperEditorSubsystem::ConvertCsvToControlRigSection(const FFilePath& InputFile, UMovieSceneControlRigParameterSection* Target, const TArray<URigMapperDefinition*>& Definitions)
{
	TArray<FString> CurveNames;
	TArray<FFrameTime> FrameTimes;
	FFrameValues CurveValuesPerFrame;
	
	if (!LoadCurveValuesFromCsv(InputFile, CurveNames, FrameTimes, CurveValuesPerFrame))
	{
		return false;
	}

	return ConvertCurveValuesToControlRigSection(CurveNames, FrameTimes, CurveValuesPerFrame, Target, Definitions);
}

UMovieSceneControlRigParameterSection* URigMapperEditorSubsystem::ConvertCsvToControlRigSectionNew(const FFilePath& InputFile, USkeletalMesh* TargetMesh, const TArray<URigMapperDefinition*>& Definitions, const FFrameRate& FrameRate, const TSubclassOf<UControlRig>& ControlRigClass, const FDirectoryPath& NewAssetPath, const FName NewAssetName)
{
	UMovieSceneControlRigParameterSection* NewSection = CreateControlRigSequence(NewAssetPath.Path, NewAssetName.ToString(), FrameRate, ControlRigClass, TargetMesh);

	if (ConvertCsvToControlRigSection(InputFile, NewSection, Definitions))
	{
		return NewSection;
	}
	return nullptr;
}

bool URigMapperEditorSubsystem::ConvertControlRigSectionToCsv(const UMovieSceneControlRigParameterSection* Source, const FFilePath& OutputFile, const TArray<URigMapperDefinition*>& Definitions, bool bOutputIntermediateCsvFiles)
{
	TArray<FString> CurveNames;
	TArray<FFrameTime> FrameTimes;
	FFrameValues CurveValuesPerFrame;

	if (Definitions.IsEmpty())
	{
		return false;
	}
	if (!LoadCurveValuesFromControlRigSection(Source, CurveNames, FrameTimes, CurveValuesPerFrame, Definitions[0]->Inputs))
	{
		return false;
	}
	
	return ConvertCurveValuesToCsv(CurveNames, FrameTimes, CurveValuesPerFrame, OutputFile, Definitions, bOutputIntermediateCsvFiles);
}

bool URigMapperEditorSubsystem::ConvertControlRigSectionToAnimSequence(const UMovieSceneControlRigParameterSection* Source, UAnimSequence* Target, const TArray<URigMapperDefinition*>& Definitions)
{
	TArray<FString> CurveNames;
	TArray<FFrameTime> FrameTimes;
	FFrameValues CurveValuesPerFrame;

	if (Definitions.IsEmpty())
	{
		return false;
	}
	if (!LoadCurveValuesFromControlRigSection(Source, CurveNames, FrameTimes, CurveValuesPerFrame, Definitions[0]->Inputs))
	{
		return false;
	}

	return ConvertCurveValuesToAnimSequence(CurveNames, FrameTimes, CurveValuesPerFrame, Target, Definitions);
}

UAnimSequence* URigMapperEditorSubsystem::ConvertControlRigSectionToAnimSequenceNew(const UMovieSceneControlRigParameterSection* Source, USkeletalMesh* TargetMesh, const TArray<URigMapperDefinition*>& Definitions, const FDirectoryPath& NewAssetPath, const FName NewAssetName)
{
	ULevelSequence* Sequence = GetSequenceFromSection(Source);
	if (!Sequence || !Sequence->GetMovieScene())
	{
		return nullptr;
	}
	
	UAnimSequence* NewSequence = CreateNewAnimSequence(NewAssetName.ToString(), NewAssetPath.Path, TargetMesh);
	SetAnimSequenceRate(NewSequence, Sequence->GetMovieScene()->GetDisplayRate(), true);
	
	TArray<FString> CurveNames;
	TArray<FFrameTime> FrameTimes;
	FFrameValues CurveValuesPerFrame;

	if (Definitions.IsEmpty())
	{
		return nullptr;
	}
	if (!LoadCurveValuesFromControlRigSection(Source, CurveNames, FrameTimes, CurveValuesPerFrame, Definitions[0]->Inputs))
	{
		return nullptr;
	}

	if (!ConvertCurveValuesToAnimSequence(CurveNames, FrameTimes, CurveValuesPerFrame, NewSequence, Definitions))
	{
		return nullptr;
	}
	NewSequence->PostLoad();
	return NewSequence;
}

bool URigMapperEditorSubsystem::LoadCurveValuesFromCsv(const FFilePath& InputFile, TArray<FString>& CurveNames, TArray<FFrameTime>& FrameTimes, FFrameValues& CurveValuesPerFrame)
{
	FFramePoses Poses;
	
	TArray<FString> CsvLines;
	
	if (!FFileHelper::LoadFileToStringArray(CsvLines, *InputFile.FilePath))
	{
		return false;
	}

	CsvLines.RemoveAt(0);
	
	int32 PoseIndex = INDEX_NONE;
	FFrameTime LastFrameNumber(0);
	FString LastCurveName;

	int32 NumberCurves = 1;

	// todo: mem
	for (const FString& Line : CsvLines)
	{
		TArray<FString> Parts;

		// todo: namespace
		if (Line.Replace(TEXT(" "), TEXT("")).ParseIntoArray(Parts, TEXT(",")) != 3)
		{
			// todo: warn
			continue;
		}
		const FString& CurveName = Parts[0];
		const FFrameNumber FrameNumber = FCString::Atoi(*Parts[1]);
		const float CurveValue = FCString::Atof(*Parts[2]);

		if (CurveName != LastCurveName)
		{
			CurveNames.AddUnique(CurveName);
			LastCurveName = CurveName;
		}

		if (FrameNumber != LastFrameNumber || PoseIndex == INDEX_NONE)
		{
			PoseIndex = FrameTimes.Find(FFrameTime(FrameNumber));
			LastFrameNumber = FrameNumber;
		}

		if (PoseIndex == INDEX_NONE)
		{
			PoseIndex = FrameTimes.Add(FrameNumber);
			Poses.AddDefaulted_GetRef().Reserve(NumberCurves);
		}
		
		Poses[PoseIndex].Add(CurveName, CurveValue);
		
		if (Poses[PoseIndex].Num() > NumberCurves)
		{
			NumberCurves = Poses[PoseIndex].Num();
		}
	}
	
	return BakeSparseKeys(Poses, CurveNames, FrameTimes, CurveValuesPerFrame);
}

bool URigMapperEditorSubsystem::WriteCurveValuesToCsv(const FFilePath& OutputFile, const TArray<FString>& CurveNames, const TArray<FFrameTime>& FrameTimes, const FFrameValues& CurveValuesPerFrame)
{
	TArray<FString> CsvLines;
	CsvLines.Add(TEXT("curve_name, frame_number, value"));
	
	for (int32 FrameIndex = 0; FrameIndex < FrameTimes.Num(); FrameIndex++)
	{
		for (int32 CurveIndex = 0; CurveIndex < CurveNames.Num(); CurveIndex++)
		{
			if (CurveValuesPerFrame[FrameIndex][CurveIndex].IsSet())
			{
				CsvLines.Add(FString::Printf(TEXT("%s, %d, %f"), *CurveNames[CurveIndex], FrameTimes[FrameIndex].RoundToFrame().Value, CurveValuesPerFrame[FrameIndex][CurveIndex].GetValue()));
			}
		}
	}
	
	return FFileHelper::SaveStringArrayToFile(CsvLines, *OutputFile.FilePath);
}

bool URigMapperEditorSubsystem::BakeSparseKeys(const FFramePoses& Poses, const TArray<FString>& CurveNames, TArray<FFrameTime>& FrameTimes, FFrameValues& CurveValuesPerFrame)
{
	if (Poses.IsEmpty())
	{
		UE_LOGF(LogRigMapperEditor, Error, "Failed to bake sparse keys: not enough poses")
		return false;
	}
	if (CurveNames.IsEmpty())
	{
		UE_LOGF(LogRigMapperEditor, Error, "Failed to bake sparse keys: not enough curves")
		return false;
	}
	if (Poses.Num() != FrameTimes.Num())
	{
		UE_LOGF(LogRigMapperEditor, Error, "Number of poses does not match number of frame times (%d vs %d)", Poses.Num(), FrameTimes.Num())
		return false;
	}
	
	// This extra work is needed to make sure frame times and thus resulting curve values are in order
	TArray<FFrameTime> FrameTimesInOrder = FrameTimes;
	FrameTimesInOrder.Sort();
	
	CurveValuesPerFrame.Reset(FrameTimesInOrder.Num());
	for (int32 FrameIndex = 0; FrameIndex < FrameTimes.Num(); FrameIndex += 1)
	{
		CurveValuesPerFrame.AddDefaulted_GetRef().AddZeroed(CurveNames.Num());
	}

	TArray<FString> BakedCurves;
	BakedCurves.Reserve(CurveNames.Num());

	// If all curves are present on frame 0, we won't go further than one loop
	for (int32 FrameIndex = 0; FrameIndex < FrameTimes.Num() && BakedCurves.Num() < CurveNames.Num(); FrameIndex += 1)
	{
		const int32 ActualFrameIndex = FrameTimes.Find(FrameTimesInOrder[FrameIndex]); // Lookup for non ordered frames
		
		for (const TPair<FString, float>& Keyframe : Poses[ActualFrameIndex])
		{
			const FString& CurveName = Keyframe.Key;
			
			if (BakedCurves.Contains(CurveName))
			{
				continue;	
			}
			BakedCurves.Add(CurveName);

			const float CurveValue = Keyframe.Value;
			const int32 CurveIndex = CurveNames.Find(CurveName);

			SparseBakeCurve(CurveName, CurveIndex, CurveValue, CurveValuesPerFrame, FrameTimesInOrder, FrameIndex, Poses, FrameTimes, ActualFrameIndex);
			
			if (BakedCurves.Num() == CurveNames.Num())
			{
				break;
			}
		}
	}

	FrameTimes = FrameTimesInOrder;
	
	if (CurveValuesPerFrame.Num() != FrameTimes.Num())
	{
		UE_LOGF(LogRigMapperEditor, Error, "Number of baked frames does not match number of frame times (%d vs %d)", CurveValuesPerFrame.Num(), FrameTimes.Num())
		return false;
	}
	if (CurveValuesPerFrame[0].Num() != CurveNames.Num())
	{
		UE_LOGF(LogRigMapperEditor, Error, "Number of curves in first frame does not match expected number of curves (%d vs %d)", CurveValuesPerFrame[0].Num(), CurveNames.Num())
		return false;
	}
	if (CurveValuesPerFrame.Last().Num() != CurveNames.Num())
	{
		UE_LOGF(LogRigMapperEditor, Error, "Number of curves in last frame does not match expected number of curves (%d vs %d)", CurveValuesPerFrame.Last().Num(), CurveNames.Num())
		return false;
	}
	return true;
}

void URigMapperEditorSubsystem::SparseBakeCurve(const FString& CurveName, int32 CurveIndex, float CurveValue,
	FFrameValues& CurveValuesPerFrame, const TArray<FFrameTime>& FrameTimesInOrder, int32 FrameIndex,
	const FFramePoses& Poses, const TArray<FFrameTime>& FrameTimes, const int32 ActualFrameIndex)
{
	// We haven't seen this control so far. So keep it const on all previous frames as well as the current one
	for (int32 BakeFrameIndex = 0; BakeFrameIndex <= FrameIndex; BakeFrameIndex += 1)
	{
		CurveValuesPerFrame[BakeFrameIndex][CurveIndex] = CurveValue;
	}
	
	float LastKeyedValue = CurveValue;
	int32 LastKeyedFrameIndex = FrameIndex;
	int32 ActualLastKeyedFrameIndex = ActualFrameIndex; // Lookup for non ordered frames

	// For following frames, add a key a lerp all previously missed keys if need be
	for (int32 BakeFrameIndex = FrameIndex + 1; BakeFrameIndex < FrameTimes.Num(); BakeFrameIndex += 1)
	{
		const int32 ActualBakeFrameIndex = FrameTimes.Find(FrameTimesInOrder[BakeFrameIndex]); // Lookup for non ordered frames

		// If the current frame has a key for the control, set it, and try to lerp all the keys we might have missed since last time we saw the control
		if (Poses[ActualBakeFrameIndex].Contains(CurveName))
		{
			// Key this frame
			CurveValuesPerFrame[BakeFrameIndex][CurveIndex] = Poses[ActualBakeFrameIndex][CurveName];
			LastKeyedValue = CurveValuesPerFrame[BakeFrameIndex][CurveIndex].Get(0);
			
			// If we have not seen this control on the previous frame, lerp all keys with missed
			if (BakeFrameIndex > LastKeyedFrameIndex + 1)
			{
				// Lerp from the last key we found for this control to the current key, adding a lerp key for each frame
				for (int32 LerpFrameIndex = LastKeyedFrameIndex + 1; LerpFrameIndex < BakeFrameIndex; LerpFrameIndex += 1)
				{
					const double LerpAlpha = (FrameTimesInOrder[LerpFrameIndex].AsDecimal() - FrameTimesInOrder[LastKeyedFrameIndex].AsDecimal()) / (FrameTimesInOrder[BakeFrameIndex].AsDecimal() - FrameTimesInOrder[LastKeyedFrameIndex].AsDecimal());
					const double LerpValue = FMath::Lerp(Poses[ActualLastKeyedFrameIndex][CurveName], Poses[ActualBakeFrameIndex][CurveName], LerpAlpha);

					CurveValuesPerFrame[LerpFrameIndex][CurveIndex] = LerpValue;
				}	
			}
			LastKeyedFrameIndex = BakeFrameIndex;
			ActualLastKeyedFrameIndex = ActualBakeFrameIndex; // Lookup for non ordered frames
		}
		else
		{
			// Per default, add a const key to all next frames (in case there isn't any key next).
			// If it happens that there is one later, LastKeyedFrameIndex has not changed, and the above will lerp and override all const keys 
			CurveValuesPerFrame[BakeFrameIndex][CurveIndex] = LastKeyedValue;
		}
	}
}

#undef LOCTEXT_NAMESPACE
