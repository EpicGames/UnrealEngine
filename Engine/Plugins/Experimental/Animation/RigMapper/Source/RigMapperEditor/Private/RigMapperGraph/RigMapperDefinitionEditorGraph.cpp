// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigMapperDefinitionEditorGraph.h"
#include "RigMapperDefinition.h"
#include "RigMapperDefinitionEditorGraphNode.h"
#include "RigMapperDefinitionEditorGraphSchema.h"
#include "RigMapperUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigMapperDefinitionEditorGraph)

static UClass* GetGraphNodeClassForFeatureType(ERigMapperFeatureType InNodeType)
{
	switch (InNodeType)
	{
	case ERigMapperFeatureType::Input:       return URigMapperDefinitionEditorGraphNode_Input::StaticClass();
	case ERigMapperFeatureType::WeightedSum: return URigMapperDefinitionEditorGraphNode_WeightedSum::StaticClass();
	case ERigMapperFeatureType::SDK:         return URigMapperDefinitionEditorGraphNode_SDK::StaticClass();
	case ERigMapperFeatureType::Multiply:    return URigMapperDefinitionEditorGraphNode_Multiply::StaticClass();
	case ERigMapperFeatureType::MathOp:		 return URigMapperDefinitionEditorGraphNode_MathOp::StaticClass();
	case ERigMapperFeatureType::Output:      return URigMapperDefinitionEditorGraphNode_Output::StaticClass();
	case ERigMapperFeatureType::NullOutput:  return URigMapperDefinitionEditorGraphNode_NullOutput::StaticClass();
	default:                                 return URigMapperDefinitionEditorGraphNode::StaticClass();
	}
}

URigMapperDefinitionEditorGraph::URigMapperDefinitionEditorGraph(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	Schema = URigMapperDefinitionEditorGraphSchema::StaticClass();
}

void URigMapperDefinitionEditorGraph::Initialize(URigMapperDefinition* InDefinition)
{
	WeakDefinition = InDefinition;
}

void URigMapperDefinitionEditorGraph::ApplyToDefinition()
{
	if (!WeakDefinition.IsValid())
	{
		return;
	}
	URigMapperDefinition* Definition = WeakDefinition.Get();

	// Clear existing Definition data
	Definition->Inputs.Empty();
	Definition->Features.Multiply.Empty();
	Definition->Features.WeightedSums.Empty();
	Definition->Features.SDKs.Empty();
	Definition->Features.MathOps.Empty();
	Definition->Outputs.Empty();
	Definition->NullOutputs.Empty();

	// Create definition features from graph nodes
	for (UEdGraphNode* Node : Nodes)
	{
		if (URigMapperDefinitionEditorGraphNode* RigMapperNode = Cast<URigMapperDefinitionEditorGraphNode>(Node))
		{
			RigMapperNode->ApplyToDefinition(Definition);
		}
	}
}

void URigMapperDefinitionEditorGraph::RebuildGraph()
{
	RemoveAllNodes();
	ConstructNodes();
	NotifyGraphChanged();
	RequestRefreshLayout(true);
	OnGraphStructureUpdated.Broadcast();
}

void URigMapperDefinitionEditorGraph::ConstructNodes()
{
	if (!WeakDefinition.IsValid())
	{
		return;
	}
	URigMapperDefinition* Definition = WeakDefinition.Get();
	
	for (const TPair<FString, FString>& Output : Definition->Outputs)
	{
		ConstructGraphNodesRec(Definition, Output.Key, true);
	}

	// Generate nodes not related to any output
	TArray<FString> FeatureNames;
	Definition->Features.GetFeatureNames(FeatureNames);
	for (const FString& Feature : FeatureNames)
	{
		if (!WSNodes.Contains(Feature) 
			&& !SDKNodes.Contains(Feature) 
			&& !MathNodes.Contains(Feature) 
			&& !MultiplyNodes.Contains(Feature))
		{
			ConstructGraphNodesRec(Definition, Feature, false);
		}
	}
	for (const FString& Input : Definition->Inputs)
	{
		if (!InputNodes.Contains(Input))
		{
			ConstructGraphNode(Input, ERigMapperFeatureType::Input);
		}
	}
	for (const FString& NullOutput : Definition->NullOutputs)
	{
		if (!NullOutputNodes.Contains(NullOutput))
		{
			ConstructGraphNode(NullOutput, ERigMapperFeatureType::NullOutput);
		}
	}
}

TArray<URigMapperDefinitionEditorGraphNode*> URigMapperDefinitionEditorGraph::GetNodesByName(const TArray<FString>& Inputs, const TArray<FString>& Features, const TArray<FString>& Outputs, const TArray<FString>& NullOutputs) const
{
	TArray<URigMapperDefinitionEditorGraphNode*> OutNodes;
	OutNodes.Reserve(Inputs.Num() + Features.Num() + Outputs.Num() + NullOutputs.Num());

	auto AddNodesByNameFromMap = [&OutNodes](const TArray<FString>& InArray, TMap<FString, URigMapperDefinitionEditorGraphNode*> InMap)
	{
		for (const FString& NodeName : InArray)
		{
			if (URigMapperDefinitionEditorGraphNode** Node = InMap.Find(NodeName))
			{
				OutNodes.Add(*Node);
			}
		}
	};

	AddNodesByNameFromMap(Inputs, InputNodes);
	AddNodesByNameFromMap(Features, SDKNodes);
	AddNodesByNameFromMap(Features, WSNodes);
	AddNodesByNameFromMap(Features, MultiplyNodes);
	AddNodesByNameFromMap(Features, MathNodes);
	AddNodesByNameFromMap(Outputs, OutputNodes);
	AddNodesByNameFromMap(NullOutputs, NullOutputNodes);

	return OutNodes;
}

URigMapperDefinitionEditorGraphNode* URigMapperDefinitionEditorGraph::FindFeatureNodeByName(const FString& InNodeName)
{
	if (URigMapperDefinitionEditorGraphNode* FoundNode = WSNodes.FindRef(InNodeName))
	{
		return FoundNode;
	}
	if (URigMapperDefinitionEditorGraphNode* FoundNode = SDKNodes.FindRef(InNodeName))
	{
		return FoundNode;
	}
	if (URigMapperDefinitionEditorGraphNode* FoundNode = MultiplyNodes.FindRef(InNodeName))
	{
		return FoundNode;
	}
	if (URigMapperDefinitionEditorGraphNode* FoundNode = MathNodes.FindRef(InNodeName))
	{
		return FoundNode;
	}
	if (URigMapperDefinitionEditorGraphNode* FoundNode = InputNodes.FindRef(InNodeName))
	{
		return FoundNode;
	}
	return nullptr;
}

URigMapperDefinitionEditorGraphNode* URigMapperDefinitionEditorGraph::FindOutputNodeByName(const FString& InNodeName)
{
	if (URigMapperDefinitionEditorGraphNode* FoundNode = OutputNodes.FindRef(InNodeName))
	{
		return FoundNode;
	}
	if (URigMapperDefinitionEditorGraphNode* FoundNode = NullOutputNodes.FindRef(InNodeName))
	{
		return FoundNode;
	}
	return nullptr;
}

TArray<FString> URigMapperDefinitionEditorGraph::GetNodesByType(ERigMapperFeatureType InNodeType) const
{
	TArray<FString> OutNodeNames;
	switch (InNodeType)
	{
	case ERigMapperFeatureType::Input:
		InputNodes.GenerateKeyArray(OutNodeNames);
		break;
	case ERigMapperFeatureType::WeightedSum:
		WSNodes.GenerateKeyArray(OutNodeNames);
		break;
	case ERigMapperFeatureType::SDK:
		SDKNodes.GenerateKeyArray(OutNodeNames);
		break;
	case ERigMapperFeatureType::Multiply:
		MultiplyNodes.GenerateKeyArray(OutNodeNames);
		break;
	case ERigMapperFeatureType::MathOp:
		MathNodes.GenerateKeyArray(OutNodeNames);
		break;
	case ERigMapperFeatureType::Output:
		OutputNodes.GenerateKeyArray(OutNodeNames);
		break;
	case ERigMapperFeatureType::NullOutput:
		NullOutputNodes.GenerateKeyArray(OutNodeNames);
		break;
	default:
		break;
	}
	return OutNodeNames;
}

bool URigMapperDefinitionEditorGraph::UpdateNodeNameInMap(const FString& InOldName, const FString& InNewName, const ERigMapperFeatureType InNodeType)
{
	switch (InNodeType)
	{
	case ERigMapperFeatureType::WeightedSum:
		if (URigMapperDefinitionEditorGraphNode* FoundNode = WSNodes.FindRef(InOldName))
		{
			WSNodes.Add(InNewName, FoundNode);
			WSNodes.Remove(InOldName);
			return true;
		}
		break;
	case ERigMapperFeatureType::SDK:
		if (URigMapperDefinitionEditorGraphNode* FoundNode = SDKNodes.FindRef(InOldName))
		{
			SDKNodes.Add(InNewName, FoundNode);
			SDKNodes.Remove(InOldName);
			return true;
		}
		break;
	case ERigMapperFeatureType::Multiply:
		if (URigMapperDefinitionEditorGraphNode* FoundNode = MultiplyNodes.FindRef(InOldName))
		{
			MultiplyNodes.Add(InNewName, FoundNode);
			MultiplyNodes.Remove(InOldName);
			return true;
		}
		break;
	case ERigMapperFeatureType::MathOp:
		if (URigMapperDefinitionEditorGraphNode* FoundNode = MathNodes.FindRef(InOldName))
		{
			MathNodes.Add(InNewName, FoundNode);
			MathNodes.Remove(InOldName);
			return true;
		}
		break;
	case ERigMapperFeatureType::Input:
		if (URigMapperDefinitionEditorGraphNode* FoundNode = InputNodes.FindRef(InOldName))
		{
			InputNodes.Add(InNewName, FoundNode);
			InputNodes.Remove(InOldName);
			return true;
		}
		break;
	case ERigMapperFeatureType::Output:
		if (URigMapperDefinitionEditorGraphNode* FoundNode = OutputNodes.FindRef(InOldName))
		{
			OutputNodes.Add(InNewName, FoundNode);
			OutputNodes.Remove(InOldName);
			return true;
		}
		break;
	case ERigMapperFeatureType::NullOutput:
		if (URigMapperDefinitionEditorGraphNode* FoundNode = NullOutputNodes.FindRef(InOldName))
		{
			NullOutputNodes.Add(InNewName, FoundNode);
			NullOutputNodes.Remove(InOldName);
			return true;
		}
		break;
	default:
		break;
	}
	return false;
}

bool URigMapperDefinitionEditorGraph::RemoveNodeFromMap(const FString& InNodeName, const ERigMapperFeatureType InNodeType)
{
	switch (InNodeType)
	{
	case ERigMapperFeatureType::WeightedSum:
		WSNodes.Remove(InNodeName);
		break;
	case ERigMapperFeatureType::SDK:
		SDKNodes.Remove(InNodeName);
		break;
	case ERigMapperFeatureType::Multiply:
		MultiplyNodes.Remove(InNodeName);
		break;
	case ERigMapperFeatureType::MathOp:
		MathNodes.Remove(InNodeName);
		break;
	case ERigMapperFeatureType::Input:
		InputNodes.Remove(InNodeName);
		break;
	case ERigMapperFeatureType::Output:
		OutputNodes.Remove(InNodeName);
		break;
	case ERigMapperFeatureType::NullOutput:
		NullOutputNodes.Remove(InNodeName);
		break;
	default:
		return false;
	}
	return true;
}

void URigMapperDefinitionEditorGraph::RebuildNodeMaps()
{
	InputNodes.Reset();
	SDKNodes.Reset();
	WSNodes.Reset();
	MultiplyNodes.Reset();
	MathNodes.Reset();
	OutputNodes.Reset();
	NullOutputNodes.Reset();
	for (UEdGraphNode* Node : Nodes)
	{
		if (URigMapperDefinitionEditorGraphNode* RMNode =
			Cast<URigMapperDefinitionEditorGraphNode>(Node))
		{
			AddNodeToMap(RMNode->GetNodeType(), RMNode->NodeName, RMNode);
		}
	}
}

FVector2D URigMapperDefinitionEditorGraph::GetNextNodeSlotPosition(ERigMapperFeatureType InNodeType) const
{
	constexpr double NodeSpacingY = 25.0;
	constexpr double InputMarginX = 50.0;
	constexpr double OutputMarginX = 50.0;
	constexpr double DefaultOutputOffsetX = 400.0;

	const bool bIsOutput = InNodeType != ERigMapperFeatureType::Input; // Treat all node types except input as output
	
	const TMap<FString, URigMapperDefinitionEditorGraphNode*>& RelevantNodes = bIsOutput ? OutputNodes : InputNodes;
	
	double MaxBottomY = 0.0;
	bool bFoundAny = false;
	
	auto AccumulateMaxY = [&MaxBottomY, &bFoundAny](const TMap<FString, URigMapperDefinitionEditorGraphNode*>& NodeMap)
	{
		for (const TPair<FString, URigMapperDefinitionEditorGraphNode*>& Pair : NodeMap)
		{
			const double BottomY = Pair.Value->NodePosY
				+ Pair.Value->GetDimensions().Y
				+ Pair.Value->GetMargin().Y;
			MaxBottomY = FMath::Max(MaxBottomY, BottomY);
			bFoundAny = true;
		}
	};

	AccumulateMaxY(RelevantNodes);
	if (bIsOutput)
	{
		AccumulateMaxY(NullOutputNodes);
	}

	const double PosY = bFoundAny ? MaxBottomY + NodeSpacingY : 0.0;
	double PosX = 0.0;
	if (bIsOutput)
	{
		if (!OutputNodes.IsEmpty())
		{
			PosX = OutputNodes.CreateConstIterator().Value()->NodePosX;
		}
		else if (!NullOutputNodes.IsEmpty())
		{
			PosX = NullOutputNodes.CreateConstIterator().Value()->NodePosX;
		}
		else
		{
			// Because there are no output nodes, derrive position from the righmost input node
			double MaxInputRight = 0.0;
			for (const TPair<FString, URigMapperDefinitionEditorGraphNode*>& Pair : InputNodes)
			{
				const double RightEdge = Pair.Value->NodePosX + Pair.Value->GetDimensions().X + InputMarginX;
				MaxInputRight = FMath::Max(MaxInputRight, RightEdge);
			}
			PosX = MaxInputRight > 0.0 ? MaxInputRight + OutputMarginX : DefaultOutputOffsetX;
		}
	}
	return FVector2D(PosX, PosY);
}

FVector2D URigMapperDefinitionEditorGraph::GetNextNodeSlotPositionFromNode(const URigMapperDefinitionEditorGraphNode* InNode) const
{
	constexpr double NodeSpacingY = 25.0;
	double PosX = 0.0;
	double PosY = 0.0;
	if (InNode)
	{
		PosX = InNode->GetNodePosX();
		PosY = InNode->GetDimensions().Y + InNode->GetNodePosY() + NodeSpacingY;
	}
	return FVector2D(PosX, PosY);
}

void URigMapperDefinitionEditorGraph::AddNodeToMap(const ERigMapperFeatureType NodeType, const FString& NodeName, URigMapperDefinitionEditorGraphNode* Node)
{
	switch (NodeType)
	{
	case ERigMapperFeatureType::WeightedSum:
		WSNodes.Add(NodeName, Node);
		break;
	case ERigMapperFeatureType::SDK:
		SDKNodes.Add(NodeName, Node);
		break;
	case ERigMapperFeatureType::Multiply:
		MultiplyNodes.Add(NodeName, Node);
		break;
	case ERigMapperFeatureType::MathOp:
		MathNodes.Add(NodeName, Node);
		break;
	case ERigMapperFeatureType::Input:
		InputNodes.Add(NodeName, Node);
		break;
	case ERigMapperFeatureType::Output:
		OutputNodes.Add(NodeName, Node);
		break;
	case ERigMapperFeatureType::NullOutput:
		NullOutputNodes.Add(NodeName, Node);
		break;
	default:
		break;
	}
}

URigMapperDefinitionEditorGraphNode* URigMapperDefinitionEditorGraph::ConstructGraphNodesRec(URigMapperDefinition* Definition, const FString& NodeName, bool bIsOutputNode)
{
	// No need to do anything is node was already created. Output nodes should not be referenced by other nodes and should only get created once.
	if (!bIsOutputNode)
	{
		URigMapperDefinitionEditorGraphNode** ExistingNode = InputNodes.Find(NodeName);
		if (!ExistingNode)
		{
			ExistingNode = WSNodes.Find(NodeName);
		}
		if (!ExistingNode)
		{
			ExistingNode = SDKNodes.Find(NodeName);
		}
		if (!ExistingNode)
		{
			ExistingNode = MathNodes.Find(NodeName);
		}
		if (!ExistingNode)
		{
			ExistingNode = MultiplyNodes.Find(NodeName);
		}
		if (ExistingNode)
		{
			return *ExistingNode;
		}
	}

	// Node was not created (referenced by a previously created node)
	if (bIsOutputNode)
	{
		return ConstructOutputNode(Definition, NodeName);
	}
	if (Definition->Inputs.Contains(NodeName))
	{
		return ConstructGraphNode(NodeName, ERigMapperFeatureType::Input);
	}
	return ConstructFeatureNode(Definition, NodeName);
}

URigMapperDefinitionEditorGraphNode* URigMapperDefinitionEditorGraph::ConstructOutputNode(URigMapperDefinition* Definition, const FString& NodeName)
{
	if (const FString* LinkedInputName = Definition->Outputs.Find(NodeName))
	{
		URigMapperDefinitionEditorGraphNode* Node = ConstructGraphNode(NodeName, ERigMapperFeatureType::Output);
		
		if (URigMapperDefinitionEditorGraphNode* LinkedNode = ConstructGraphNodesRec(Definition, *LinkedInputName, false))
		{
			LinkGraphNodes(LinkedNode, Node);
		}
		else
		{
			// Output node is saved without connection.
			UEdGraphPin* InPin = Node->CreateInputPin();
			InPin->bHidden = false;
		}

		return Node;
	}
	return nullptr;
}

URigMapperDefinitionEditorGraphNode* URigMapperDefinitionEditorGraph::ConstructFeatureNode(URigMapperDefinition* Definition, const FString& NodeName)
{
	ERigMapperFeatureType FeatureType;

	if (const FRigMapperFeature* Feature = Definition->Features.Find(NodeName, FeatureType))
	{
		if (URigMapperDefinitionEditorGraphNode* Node = ConstructGraphNode(NodeName, FeatureType))
		{
			Node->LoadFromFeature(Feature);
			TArray<FString> FeatureInputNames;
			Feature->GetInputs(FeatureInputNames);

			for (const FString& FeatureInput : FeatureInputNames)
			{
				URigMapperDefinitionEditorGraphNode* LinkedNode = nullptr;
				if (!FeatureInput.IsEmpty())
				{
					LinkedNode = ConstructGraphNodesRec(Definition, FeatureInput, false);
				}
				if (LinkedNode)
				{
					LinkGraphNodes(LinkedNode, Node);
				}
				else
				{
					// Feature node is saved without connected input pin.
					UEdGraphPin* InPin = Node->CreateInputPin();
					InPin->bHidden = false;
				}
			}

			if (FeatureInputNames.Num() == 0)
			{
				Node->CreateInputPin();
			}
			if (FeatureInputNames.Num() <= 1 && FeatureType == ERigMapperFeatureType::Multiply)
			{
				Node->CreateInputPin();
			}

			Node->UpdateNodeFields();
			return Node;
		}
	}
	return nullptr;
}

void URigMapperDefinitionEditorGraph::LinkGraphNodes(URigMapperDefinitionEditorGraphNode* InNode, URigMapperDefinitionEditorGraphNode* OutNode)
{
	if (InNode != OutNode)
	{
		UEdGraphPin* InPin = OutNode->CreateInputPin();
		InPin->bHidden = false;

		UEdGraphPin* OutPin = InNode->GetOutputPin();
		if (!OutPin)
		{
			OutPin = InNode->CreateOutputPin();
			OutPin->bHidden = false;
		}
		if (OutPin)
		{
			OutPin->MakeLinkTo(InPin);
		}
	}
}
		

URigMapperDefinitionEditorGraphNode* URigMapperDefinitionEditorGraph::ConstructGraphNode(const FString& NodeName, const ERigMapperFeatureType NodeType)
{
	if (!WeakDefinition.IsValid())
	{
		return nullptr;
	}
	
	constexpr bool bSelectNewNode = false;
	
	URigMapperDefinitionEditorGraphNode* Node = NewObject<URigMapperDefinitionEditorGraphNode>(
		this, GetGraphNodeClassForFeatureType(NodeType), NAME_None, RF_Transactional);
	this->AddNode(Node, /*bUserAction=*/ false, /*bSelectNewNode=*/ bSelectNewNode);
	Node->CreateNewGuid();
	Node->PostPlacedNewNode();

	Node->SetupNode(NodeName, NodeType);
	if (NodeType != ERigMapperFeatureType::NullOutput && NodeType != ERigMapperFeatureType::Output)
	{
		// One output node must be created for non output types, even if node is "hanging".
		UEdGraphPin* OutPin = Node->GetOutputPin();
		if (!OutPin)
		{
			OutPin = Node->CreateOutputPin();
			OutPin->bHidden = false;
		}
	}
	AddNodeToMap(NodeType, NodeName, Node);
	
	return Node;
}


void URigMapperDefinitionEditorGraph::GetExistingNodeNames(const ERigMapperFeatureType InNodeType, TArray<FString>& OutExistingNames)
{
	OutExistingNames.Reset();
	if (InNodeType == ERigMapperFeatureType::Output || InNodeType == ERigMapperFeatureType::NullOutput)
	{
		OutExistingNames.Reserve(OutputNodes.Num() + NullOutputNodes.Num());
		for (const TPair<FString, URigMapperDefinitionEditorGraphNode*>& Pair : OutputNodes) { OutExistingNames.Add(Pair.Key); }
		for (const TPair<FString, URigMapperDefinitionEditorGraphNode*>& Pair : NullOutputNodes) { OutExistingNames.Add(Pair.Key); }
	}
	else
	{
		OutExistingNames.Reserve(InputNodes.Num() + WSNodes.Num() + SDKNodes.Num() + MultiplyNodes.Num() + MathNodes.Num());
		for (const TPair<FString, URigMapperDefinitionEditorGraphNode*>& Pair : InputNodes) { OutExistingNames.Add(Pair.Key); }
		for (const TPair<FString, URigMapperDefinitionEditorGraphNode*>& Pair : WSNodes) { OutExistingNames.Add(Pair.Key); }
		for (const TPair<FString, URigMapperDefinitionEditorGraphNode*>& Pair : SDKNodes) { OutExistingNames.Add(Pair.Key); }
		for (const TPair<FString, URigMapperDefinitionEditorGraphNode*>& Pair : MathNodes) { OutExistingNames.Add(Pair.Key); }
		for (const TPair<FString, URigMapperDefinitionEditorGraphNode*>& Pair : MultiplyNodes) { OutExistingNames.Add(Pair.Key); }
	}
}

URigMapperDefinitionEditorGraphNode* URigMapperDefinitionEditorGraph::CreateGraphNode(const ERigMapperFeatureType InNodeType, UEdGraphPin* InFromPin, const FVector2f& InLocation, bool bSelectNewNode /*= true*/, const FString InDesiredNodeName)
{
	if (!WeakDefinition.IsValid())
	{
		return nullptr;
	}

	if (!InDesiredNodeName.IsEmpty())
	{
		// DesiredNodeName is expected to be unique. If it is not, no node should be created.
		if (InNodeType == ERigMapperFeatureType::Output || InNodeType == ERigMapperFeatureType::NullOutput)
		{
			if (FindOutputNodeByName(InDesiredNodeName))
			{
				return nullptr;
			}
		}
		else
		{
			if (FindFeatureNodeByName(InDesiredNodeName))
			{
				return nullptr;
			}
		}
	}
	FString FromPinName = TEXT("");
	if (InFromPin && InFromPin->GetOwningNode())
	{
		FromPinName = Cast<URigMapperDefinitionEditorGraphNode>(InFromPin->GetOwningNode())->NodeName;
	}
	URigMapperDefinitionEditorGraphNode* NewNode = NewObject<URigMapperDefinitionEditorGraphNode>(
		this, GetGraphNodeClassForFeatureType(InNodeType), NAME_None, RF_Transactional);
	this->AddNode(NewNode, /*bUserAction=*/ false, /*bSelectNewNode=*/ bSelectNewNode);
	NewNode->CreateNewGuid();
	NewNode->PostPlacedNewNode();
	NewNode->NodePosX = InLocation.X;
	NewNode->NodePosY = InLocation.Y;
	// Create unique feature (node) name.
	FString NodeName = InDesiredNodeName;
	if (NodeName == TEXT(""))
	{
		// Generate new unique node name from pin (if provided).
		if (InNodeType != ERigMapperFeatureType::Input && InNodeType != ERigMapperFeatureType::Output && InNodeType != ERigMapperFeatureType::NullOutput)
		{
			// Only feature types can propagate names in chains.
			NodeName = FromPinName;
		}
		TArray<FString> ExistingNames;
		GetExistingNodeNames(InNodeType, ExistingNames);
		NodeName = FRigMapperUtils::GenerateUniqueFeatureName(ExistingNames, NodeName, InNodeType);
	}
	if (InNodeType != ERigMapperFeatureType::Input && InNodeType != ERigMapperFeatureType::NullOutput)
	{
		UEdGraphPin* InPin = NewNode->CreateInputPin();
		InPin->bHidden = false;
	}
	FString InputNodeName = TEXT("");
	if (InFromPin && InFromPin->Direction == EGPD_Output)
	{
		InputNodeName = FromPinName;
	}
	if (InNodeType != ERigMapperFeatureType::Output && InNodeType != ERigMapperFeatureType::NullOutput)
	{
		UEdGraphPin* OutPin = NewNode->CreateOutputPin();
		OutPin->bHidden = false;
	}
	if (InNodeType == ERigMapperFeatureType::WeightedSum)
	{
		if (URigMapperDefinitionEditorGraphNode_WeightedSum* WsNode = Cast<URigMapperDefinitionEditorGraphNode_WeightedSum>(NewNode))
		{
			WsNode->WeightedInputs.Add(FRigMapperWsInput(InputNodeName, 0.0));
		}
	}
	else if (InNodeType == ERigMapperFeatureType::Multiply)
	{
		// Multiply features require at least 2 inputs.
		UEdGraphPin* InPin = NewNode->CreateInputPin();
		InPin->bHidden = false;
	}
	else if (InNodeType == ERigMapperFeatureType::MathOp)
	{
		if (URigMapperDefinitionEditorGraphNode_MathOp* MathNode = Cast<URigMapperDefinitionEditorGraphNode_MathOp>(NewNode))
		{
			FRigMapperMathInput& MathInput = MathNode->Inputs.AddDefaulted_GetRef();
			MathInput.NodeName = InputNodeName;

			// Default operation (Min) requires 2 inputs
			UEdGraphPin* InPin = NewNode->CreateInputPin();
			InPin->bHidden = false;
			MathNode->Inputs.AddDefaulted();
		}
	}
	NewNode->SetupNode(NodeName, InNodeType);
	AddNodeToMap(InNodeType, NodeName, NewNode);

	if (InFromPin)
	{
		const UEdGraphSchema* GraphSchema = GetSchema();
		if (InFromPin->Direction == EGPD_Output)
		{
			const TArray<UEdGraphPin*>& Pins = NewNode->GetInputPins();
			if (Pins.Num() > 0)
			{
				InFromPin->MakeLinkTo(Pins[0]);
			}
		}
		else
		{
			UEdGraphPin* OutPin = NewNode->GetOutputPin();
			if (OutPin)
			{
				InFromPin->BreakAllPinLinks();
				OutPin->MakeLinkTo(InFromPin);
				Cast<URigMapperDefinitionEditorGraphNode>(InFromPin->GetOwningNode())->RelinkFeature(NodeName, InFromPin);
			}
		}
	}
	NewNode->UpdateNodeFields();
	// Notify listeners that new node is created.
	OnGraphStructureUpdated.Broadcast();
	return NewNode;
}

void URigMapperDefinitionEditorGraph::LayoutNodeRec(URigMapperDefinitionEditorGraphNode* InNode, double InputsWidth, double PosY, TArray<URigMapperDefinitionEditorGraphNode*>& LayedOutNodes) const
{
	const int32 NodeMarginX = 20;
	const int32 NodeMarginY = 5;
	
	URigMapperDefinitionEditorGraphNode* LinkedNode = nullptr;
	double SubGraphHeight = 0;
	
	for (UEdGraphPin* InPin : InNode->GetInputPins())
	{ 
		for (const UEdGraphPin* OutPin : InPin->LinkedTo)
		{
			LinkedNode = Cast<URigMapperDefinitionEditorGraphNode>(OutPin->GetOwningNode());
			if (!LayedOutNodes.Contains(LinkedNode))
			{
				LayedOutNodes.Add(LinkedNode);

				const double DesiredPosY =  PosY + SubGraphHeight;
				LayoutNodeRec(LinkedNode, InputsWidth, DesiredPosY, LayedOutNodes);
				SubGraphHeight += LinkedNode->GetDimensions().Y + LinkedNode->GetMargin().Y + (LinkedNode->NodePosY - DesiredPosY);
			}

			const double TargetPosX = LinkedNode->NodePosX + LinkedNode->GetDimensions().X + LinkedNode->GetMargin().X; 
			if (TargetPosX > InNode->NodePosX)
			{
				InNode->NodePosX = TargetPosX;
			}
		}
	}
	
	InNode->NodePosY = PosY;

	const FVector2D& Dimensions = InNode->GetDimensions();
	FVector2D Margin = { NodeMarginX, NodeMarginY };
	
	if (InNode->GetNodeType() == ERigMapperFeatureType::Input)
	{
		InNode->NodePosX = 0;
		Margin.X += InputsWidth - Dimensions.X;
	}
	else if (InNode->GetNodeType() == ERigMapperFeatureType::Output)
	{
		Margin.X = 0;
	}
	else if (InNode->GetNodeType() == ERigMapperFeatureType::NullOutput)
	{
		Margin.X = 0;
	}
	else
	{
		InNode->NodePosX = FMath::Max(InNode->NodePosX, InputsWidth);
	}
	if (SubGraphHeight > Dimensions.Y + Margin.Y)
	{
		const double Offset = SubGraphHeight / 2 - (Dimensions.Y + Margin.Y) / 2;

		Margin.Y += Offset;
		InNode->NodePosY += Offset;
	}

	InNode->SetMargin(Margin);
}

void URigMapperDefinitionEditorGraph::LayoutNodes() const
{
	double InputsMaxWidth = 0;

	const int32 InputMarginX = 50;
	for (const TPair<FString, URigMapperDefinitionEditorGraphNode*>& InputNode : InputNodes)
	{
		const FVector2D& NodeDimensions = InputNode.Value->GetDimensions();

		if (NodeDimensions.X > InputsMaxWidth)
		{
			InputsMaxWidth = NodeDimensions.X;
		}
	}
	InputsMaxWidth += InputMarginX;

	double MaxPosX = 0;
	TArray<URigMapperDefinitionEditorGraphNode*> LayedOutNodes;

	double PosY = 0;
	const int32 SubGraphMarginY = 25;
	
	for (const TPair<FString, URigMapperDefinitionEditorGraphNode*>& Output : OutputNodes)
	{
		LayoutNodeRec(Output.Value, InputsMaxWidth, PosY, LayedOutNodes);
		PosY = Output.Value->NodePosY + Output.Value->GetDimensions().Y + Output.Value->GetMargin().Y + SubGraphMarginY;
		if (Output.Value->NodePosX > MaxPosX)
		{
			MaxPosX = Output.Value->NodePosX;
		}
	}
	
	const int32 OutputMarginX = 50;
	for (const TPair<FString, URigMapperDefinitionEditorGraphNode*>& Output : OutputNodes)
	{
		Output.Value->NodePosX = MaxPosX + OutputMarginX;
		 for (UEdGraphPin* InPin : Output.Value->GetInputPins())
		 {
		 	for (const UEdGraphPin* OutPin : InPin->LinkedTo)
		 	{
		 		URigMapperDefinitionEditorGraphNode* LinkedNode = Cast<URigMapperDefinitionEditorGraphNode>(OutPin->GetOwningNode());
				
		 		if (LinkedNode && !LinkedNode->GetInputPins().IsEmpty())
		 		{
		 			LinkedNode->NodePosX = Output.Value->NodePosX - (LinkedNode->GetDimensions().X + LinkedNode->GetMargin().X + OutputMarginX);
		 		}
		 	}
		 }
	}

	// Layout nodes not related to any output
	auto LayoutOrphanNodes = [this, &LayedOutNodes, &PosY, InputsMaxWidth, SubGraphMarginY](const TMap<FString, URigMapperDefinitionEditorGraphNode*>& NodeMap, double OverridePosX = -1.0)
		{
			for (const TPair<FString, URigMapperDefinitionEditorGraphNode*>& Node : NodeMap)
			{
				if (!LayedOutNodes.Contains(Node.Value))
				{
					if (OverridePosX >= 0.0)
					{
						Node.Value->NodePosX = OverridePosX;
					}
					LayoutNodeRec(Node.Value, InputsMaxWidth, PosY, LayedOutNodes);
					PosY = Node.Value->NodePosY + Node.Value->GetDimensions().Y + Node.Value->GetMargin().Y + SubGraphMarginY;
				}
			}
		};

	LayoutOrphanNodes(SDKNodes);
	LayoutOrphanNodes(WSNodes);
	LayoutOrphanNodes(MathNodes);
	LayoutOrphanNodes(MultiplyNodes);
	LayoutOrphanNodes(InputNodes);
	LayoutOrphanNodes(NullOutputNodes, MaxPosX + OutputMarginX);
}

void URigMapperDefinitionEditorGraph::RemoveAllNodes()
{
	TArray<UEdGraphNode*> NodesToRemove = Nodes;
	
	for (int32 NodeIndex = 0; NodeIndex < NodesToRemove.Num(); ++NodeIndex)
	{
		RemoveNode(NodesToRemove[NodeIndex]);
	}

	Nodes.Reset();
	InputNodes.Reset();
	SDKNodes.Reset();
	WSNodes.Reset();
	MultiplyNodes.Reset();
	MathNodes.Reset();
	OutputNodes.Reset();
	NullOutputNodes.Reset();
}
