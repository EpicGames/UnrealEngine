// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraph.h"
#include "RigMapperDefinitionEditorGraph.generated.h"

#define UE_API RIGMAPPEREDITOR_API

enum class ERigMapperFeatureType : uint8;
class URigMapperDefinitionEditorGraphNode;
class URigMapperDefinition;

DECLARE_MULTICAST_DELEGATE(FOnGraphStructureUpdated);
DECLARE_DELEGATE_OneParam(FOnFocusNodeRequested, URigMapperDefinitionEditorGraphNode*);

/**
 * 
 */
UCLASS(MinimalAPI)
class URigMapperDefinitionEditorGraph : public UEdGraph
{
	GENERATED_UCLASS_BODY()

public:
	UE_API void Initialize(URigMapperDefinition* InDefinition);
	
	FOnGraphStructureUpdated OnGraphStructureUpdated;
	FOnFocusNodeRequested OnFocusNodeRequested;

	/**
	 * Applies graph nodes state to definition, discarding the previous state.
	 * Intended to call only on save and close so it does not call Modify() in order not to dirty the state of Definition again.
	 * If needed elsewhere, in order to preserve the undo stack, call Modify outside this method.
	 */
	UE_API void ApplyToDefinition();
	UE_API void RebuildGraph();
	void RequestRefreshLayout(bool bInRefreshLayout) { bRefreshLayout = bInRefreshLayout; }
	bool NeedsRefreshLayout() const { return bRefreshLayout; }
	UE_API void LayoutNodes() const;
	
	URigMapperDefinition* GetDefinition() const { return WeakDefinition.Get(); };

	void GetExistingNodeNames(const ERigMapperFeatureType InNodeType, TArray<FString>& OutExistingNames);

	UE_API URigMapperDefinitionEditorGraphNode* CreateGraphNode(const ERigMapperFeatureType InNodeType, UEdGraphPin* InFromPin, const FVector2f& InLocation, bool bSelectNewNode = true, const FString InDesiredNodeName = TEXT(""));
	UE_API TArray<URigMapperDefinitionEditorGraphNode*> GetNodesByName(const TArray<FString>& Inputs, const TArray<FString>& Features, const TArray<FString>& Outputs, const TArray<FString>& NullOutputs) const;
	/** Retrieves first feature node that matches the InNodeName (assumes node names are unique and includes input nodes) */
	UE_API URigMapperDefinitionEditorGraphNode* FindFeatureNodeByName(const FString& InNodeName);
	/** Retrieves first output node that matches the InNodeName (assumes output node names are unique and includes nulloutput nodes) */
	UE_API URigMapperDefinitionEditorGraphNode* FindOutputNodeByName(const FString& InNodeName);
	/** Retrieves all names of requested type nodes */
	UE_API TArray<FString> GetNodesByType(ERigMapperFeatureType InNodeType) const;
	/** Updates the key in the node map when the node is renamed (assumes new name is unique) */
	UE_API bool UpdateNodeNameInMap(const FString& InOldName, const FString& InNewName, const ERigMapperFeatureType InNodeType);
	/** Removes node from the map when deleted */
	UE_API bool RemoveNodeFromMap(const FString& InNodeName, const ERigMapperFeatureType InNodeType);
	/** Rebuilds node map from graph nodes **/
	UE_API void RebuildNodeMaps();
	/** Gets the next available node position. Input types are placed in the input column; all other types in the output column. */
	UE_API FVector2D GetNextNodeSlotPosition(ERigMapperFeatureType InNodeType) const;
	/** Gets the position for the next node below the given node **/
	UE_API FVector2D GetNextNodeSlotPositionFromNode(const URigMapperDefinitionEditorGraphNode* InNode) const;

private:
	UE_API void ConstructNodes();
	UE_API void RemoveAllNodes();
	UE_API URigMapperDefinitionEditorGraphNode* ConstructGraphNode(const FString& NodeName, const ERigMapperFeatureType NodeType);
	UE_API URigMapperDefinitionEditorGraphNode* ConstructGraphNodesRec(URigMapperDefinition* Definition, const FString& InNodeName, bool bIsOutputNode);
	UE_API URigMapperDefinitionEditorGraphNode* ConstructOutputNode(URigMapperDefinition* Definition, const FString& NodeName);
	UE_API URigMapperDefinitionEditorGraphNode* ConstructFeatureNode(URigMapperDefinition* Definition, const FString& NodeName);
	static UE_API void LinkGraphNodes(URigMapperDefinitionEditorGraphNode* InNode, URigMapperDefinitionEditorGraphNode* OutNode);
	UE_API void LayoutNodeRec(URigMapperDefinitionEditorGraphNode* InNode, double InputsWidth, double PosY, TArray<URigMapperDefinitionEditorGraphNode*>& LayedOutNodes) const;
	
	void AddNodeToMap(const ERigMapperFeatureType NodeType, const FString& NodeName, URigMapperDefinitionEditorGraphNode* Node);

private:
	TWeakObjectPtr<URigMapperDefinition> WeakDefinition;
	TMap<FString, URigMapperDefinitionEditorGraphNode*> InputNodes;
	TMap<FString, URigMapperDefinitionEditorGraphNode*> SDKNodes;
	TMap<FString, URigMapperDefinitionEditorGraphNode*> WSNodes;
	TMap<FString, URigMapperDefinitionEditorGraphNode*> MultiplyNodes;
	TMap<FString, URigMapperDefinitionEditorGraphNode*> MathNodes;
	TMap<FString, URigMapperDefinitionEditorGraphNode*> OutputNodes;
	TMap<FString, URigMapperDefinitionEditorGraphNode*> NullOutputNodes;

	bool bRefreshLayout = false;
};

#undef UE_API
