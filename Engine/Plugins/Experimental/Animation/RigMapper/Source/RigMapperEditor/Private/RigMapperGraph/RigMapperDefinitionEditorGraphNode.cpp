// Copyright Epic Games, Inc. All Rights Reserved.


#include "RigMapperDefinitionEditorGraphNode.h"

#include "SRigMapperDefinitionGraphEditorNode.h"
#include "RigMapperDefinitionEditorGraph.h"
#include "RigMapperUtils.h"
#include "PropertyEditorModule.h"
#include "EdGraph/EdGraphPin.h"
#include "SGraphNodeComment.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigMapperDefinitionEditorGraphNode)

#define LOCTEXT_NAMESPACE "RigMapperDefinitionEditorGraphNode"

TSharedPtr<SGraphNode> URigMapperDefinitionEditorGraphNode::NodeFactory::CreateNode(UEdGraphNode* Node) const
{
	if (URigMapperDefinitionEditorGraphNode* ThisNode = Cast<URigMapperDefinitionEditorGraphNode>(Node))
	{
		if (URigMapperDefinitionEditorGraph* Graph = Cast<URigMapperDefinitionEditorGraph>(Node->GetGraph()))
		{
			Graph->RequestRefreshLayout(false);
		}

		TSharedRef<SGraphNode> GraphNode = SNew(SRigMapperDefinitionGraphEditorNode, ThisNode);
		GraphNode->SlatePrepass();
		ThisNode->SetDimensions(GraphNode->GetDesiredSize());		
		return GraphNode;
	}
	if (URigMapperCommentNode* CommentNode = Cast<URigMapperCommentNode>(Node))
	{
		return SNew(SGraphNodeComment, CommentNode);
	}
	return nullptr;
}

void URigMapperDefinitionEditorGraphNode::GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const
{

}

void URigMapperDefinitionEditorGraphNode::AllocateDefaultPins()
{

}

void URigMapperDefinitionEditorGraphNode::SetupNode(const FString& InNodeName, ERigMapperFeatureType InNodeType)
{
	NodeName = InNodeName;
	NodeType = InNodeType;
	NodeTitle = FText::FromString(NodeName);	
}


void URigMapperDefinitionEditorGraphNode::PostLoad()
{
	Super::PostLoad();

	InputPins.Reset();
	OutputPins.Reset();
	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin->Direction == EGPD_Input) { InputPins.Add(Pin); }
		else if (Pin->Direction == EGPD_Output) { OutputPins.Add(Pin); }
	}
	UpdateNodeFields();
}

void URigMapperDefinitionEditorGraphNode::UpdateNodeFields()
{
	NodeTitle = FText::FromString(NodeName);
}

void URigMapperDefinitionEditorGraphNode::RelinkFeature(const FString& InNewInput, UEdGraphPin* InputPin)
{
	if (!InputPin)
	{
		return;
	}

	const int32 PinIndex = InputPins.Find(InputPin);
	if (PinIndex == INDEX_NONE) return;

	Modify();
	if (NodeType == ERigMapperFeatureType::WeightedSum)
	{
		if (URigMapperDefinitionEditorGraphNode_WeightedSum* WsNode =
			Cast<URigMapperDefinitionEditorGraphNode_WeightedSum>(this))
		{
			if (WsNode->WeightedInputs.IsValidIndex(PinIndex))
			{
				WsNode->WeightedInputs[PinIndex].Name = InNewInput;
			}
			else
			{
				// Pin exists but no WeightedInput entry, so one should be added
				WsNode->WeightedInputs.Add(FRigMapperWsInput(InNewInput, 0.0));
			}
		}
	}
	else if (NodeType == ERigMapperFeatureType::MathOp)
	{
		if (URigMapperDefinitionEditorGraphNode_MathOp* MathNode =
			Cast<URigMapperDefinitionEditorGraphNode_MathOp>(this))
		{
			if (MathNode->Inputs.IsValidIndex(PinIndex))
			{
				MathNode->Inputs[PinIndex].NodeName = InNewInput;
			}
			else
			{
				// Pin exists but no MathInput entry, so one should be added
				FRigMapperMathInput& MathInput = MathNode->Inputs.AddDefaulted_GetRef();
				MathInput.NodeName = InNewInput;
			}
		}
	}
	UpdateNodeFields();
}

void URigMapperDefinitionEditorGraphNode::AddNewInputLink(const FString& InNewInput)
{
	Modify();
	if (NodeType == ERigMapperFeatureType::WeightedSum)
	{
		if (URigMapperDefinitionEditorGraphNode_WeightedSum* WsNode =
			Cast<URigMapperDefinitionEditorGraphNode_WeightedSum>(this))
		{
			WsNode->WeightedInputs.Add(FRigMapperWsInput(InNewInput, 0.0));
		}
	}
	else if (NodeType == ERigMapperFeatureType::MathOp)
	{
		if (URigMapperDefinitionEditorGraphNode_MathOp* MathNode =
			Cast<URigMapperDefinitionEditorGraphNode_MathOp>(this))
		{
			FRigMapperMathInput& MathInput = MathNode->Inputs.AddDefaulted_GetRef();
			MathInput.NodeName = InNewInput;
		}
	}
	UpdateNodeFields();
}

void URigMapperDefinitionEditorGraphNode::RemoveLinksToFeature(const FString& InFeatureName)
{
	Modify();
	if (NodeType == ERigMapperFeatureType::WeightedSum)
	{
		if (URigMapperDefinitionEditorGraphNode_WeightedSum* WsNode =
			Cast<URigMapperDefinitionEditorGraphNode_WeightedSum>(this))
		{
			for (FRigMapperWsInput& Entry : WsNode->WeightedInputs)
			{
				if (Entry.Name == InFeatureName)
				{
					Entry.Name = TEXT("");
				}
			}
		}
	}
	else if (NodeType == ERigMapperFeatureType::MathOp)
	{
		if (URigMapperDefinitionEditorGraphNode_MathOp* MathNode =
			Cast<URigMapperDefinitionEditorGraphNode_MathOp>(this))
		{
			for (FRigMapperMathInput& Entry : MathNode->Inputs)
			{
				if (Entry.NodeName == InFeatureName)
				{
					Entry.NodeName = TEXT("");
				}
			}
		}
	}
	UpdateNodeFields();
}

void URigMapperDefinitionEditorGraphNode::PinConnectionListChanged(UEdGraphPin* Pin)
{
	if (!Pin)
	{
		return;
	}

	if (Pin->Direction == EGPD_Input && Pin->LinkedTo.Num() == 0)
	{
		RelinkFeature(TEXT(""), Pin);
	}
}

void URigMapperDefinitionEditorGraphNode::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	if (PropertyAboutToChange && PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(URigMapperDefinitionEditorGraphNode, NodeName))
	{
		PreviousNodeName = NodeName;
	}
}

void URigMapperDefinitionEditorGraphNode::PostEditChangeChainProperty(FPropertyChangedChainEvent& Event)
{
	Modify();

	Super::PostEditChangeChainProperty(Event);

	URigMapperDefinitionEditorGraph* RigMapperGraph = Cast<URigMapperDefinitionEditorGraph>(GetGraph());
	if (!RigMapperGraph)
	{
		return;
	}

	if (NodeType == ERigMapperFeatureType::WeightedSum || NodeType == ERigMapperFeatureType::MathOp)
	{
		TMap<FString, int32> PropertyNameStack;
		Event.GetArrayIndicesPerObject(0, PropertyNameStack);

		const FString InputsProperty = NodeType == ERigMapperFeatureType::WeightedSum ?
			GET_MEMBER_NAME_STRING_CHECKED(URigMapperDefinitionEditorGraphNode_WeightedSum, WeightedInputs) :
			GET_MEMBER_NAME_STRING_CHECKED(URigMapperDefinitionEditorGraphNode_MathOp, Inputs);

		const FString InputNameProperty = NodeType == ERigMapperFeatureType::WeightedSum ?
			GET_MEMBER_NAME_STRING_CHECKED(FRigMapperWsInput, Name) :
			GET_MEMBER_NAME_STRING_CHECKED(FRigMapperMathInput, NodeName);

		const int32 PinIndex = PropertyNameStack.Contains(InputsProperty) ? PropertyNameStack.FindRef(InputsProperty) : INDEX_NONE;

		const int32 MinReqPins = GetMinInputPinCount();

		if (Event.ChangeType & EPropertyChangeType::ArrayClear)
		{
			// REMOVING ALL INPUTS
			// All inputs are removed but node has to have minimal number of input pins exposed.
			for (int32 Index = InputPins.Num() - 1; Index >= MinReqPins; --Index)
			{
				RemoveInputPin(InputPins[Index]);
			}
			// Break remaining input links.
			for (UEdGraphPin* InPin : InputPins)
			{
				InPin->BreakAllPinLinks();
			}
		}
		else if (Event.ChangeType & EPropertyChangeType::ArrayRemove)
		{
			// REMOVING A SINGLE INPUT
			if (PinIndex != INDEX_NONE && InputPins.IsValidIndex(PinIndex))
			{
				if (InputPins.Num() > MinReqPins)
				{
					RemoveInputPin(InputPins[PinIndex]);
				}
				else
				{
					InputPins[PinIndex]->BreakAllPinLinks();
				}
			}
		}
		else if (Event.ChangeType & EPropertyChangeType::ArrayAdd)
		{
			// ADDING A NEW (EMPTY) INPUT
			if (PinIndex != INDEX_NONE) // This will be true only for WS and MathOp features
			{
				if (GetInputPinCount() > InputPins.Num())
				{
					CreateInputPin();
				}
			}
		}
		else if (Event.ChangeType & EPropertyChangeType::ValueSet)
		{
			if ((PinIndex != INDEX_NONE && Event.GetMemberPropertyName() == InputNameProperty))
			{
				// CHANGING AN INPUT NAME
				URigMapperDefinitionEditorGraphNode* LinkedNode = nullptr;
				const int32 InputIndex = PinIndex == INDEX_NONE ? 0 : PinIndex;
				if (InputPins.IsValidIndex(InputIndex))
				{
					InputPins[InputIndex]->BreakAllPinLinks();
					const FString InputName = GetNodeNameFromInputPin(PinIndex);
					if (!InputName.IsEmpty())
					{
						LinkedNode = RigMapperGraph->FindFeatureNodeByName(InputName);
						if (LinkedNode && LinkedNode->GetOutputPin())
						{
							InputPins[InputIndex]->MakeLinkTo(LinkedNode->GetOutputPin());
						}
					}
				}
			}
		}
	}
	if (NodeType == ERigMapperFeatureType::MathOp)
	{
		if (URigMapperDefinitionEditorGraphNode_MathOp* MathNode =
			Cast<URigMapperDefinitionEditorGraphNode_MathOp>(this))
		{
			if (Event.GetPropertyName() == GET_MEMBER_NAME_CHECKED(URigMapperDefinitionEditorGraphNode_MathOp, Operation))
			{
				const int32 MinPins = MathNode->GetMinInputPinCount();
				// Add pins up to minimum
				while (InputPins.Num() < MinPins)
				{
					CreateInputPin();
				}
				while (MathNode->Inputs.Num() < InputPins.Num())
				{
					MathNode->Inputs.AddDefaulted();
				}
				// TODO: Remove excess pins (only if they're unconnected and above minimum)
				//while (InputPins.Num() > MinPins && InputPins.Last()->LinkedTo.Num() == 0)
				//{
				//	RemoveInputPin(InputPins.Last());
				//}
			}
		}
	}
	if (Event.GetPropertyName() == GET_MEMBER_NAME_CHECKED(URigMapperDefinitionEditorGraphNode, NodeName))
	{
		if (NodeName != PreviousNodeName)
		{
			bool bDuplicateNodeName = false;
			if (NodeType == ERigMapperFeatureType::Output || NodeType == ERigMapperFeatureType::NullOutput)
			{
				bDuplicateNodeName = RigMapperGraph->FindOutputNodeByName(NodeName) != nullptr;
			}
			else
			{
				bDuplicateNodeName = RigMapperGraph->FindFeatureNodeByName(NodeName) != nullptr;
			}
			if (bDuplicateNodeName)
			{
				TArray<FString> ExistingNames;
				RigMapperGraph->GetExistingNodeNames(NodeType, ExistingNames);
				NodeName = FRigMapperUtils::GenerateUniqueFeatureName(ExistingNames, NodeName, NodeType);
			}

			if (NodeType != ERigMapperFeatureType::Output && NodeType != ERigMapperFeatureType::NullOutput) // Output and NullOutput nodes don't have output pins.
			{
				// Update inputs of all nodes directly connected to this node.
				for (UEdGraphPin* OutPin : OutputPins) // It will always be one output pin.
				{
					for (UEdGraphPin* LinkedPin : OutPin->LinkedTo)
					{
						if (URigMapperDefinitionEditorGraphNode* LinkedNode = Cast<URigMapperDefinitionEditorGraphNode>(LinkedPin->GetOwningNode()))
						{
							LinkedNode->RelinkFeature(NodeName, LinkedPin);
						}
					}
				}
			}
			RigMapperGraph->UpdateNodeNameInMap(PreviousNodeName, NodeName, NodeType);
		}
	}
	UpdateNodeFields();
}

UEdGraphPin* URigMapperDefinitionEditorGraphNode::CreateInputPin()
{
	return InputPins.Add_GetRef(CreatePin(EEdGraphPinDirection::EGPD_Input, NAME_None, NAME_None, NAME_None));
}

UEdGraphPin* URigMapperDefinitionEditorGraphNode::CreateOutputPin()
{
	return OutputPins.Add_GetRef(CreatePin(EEdGraphPinDirection::EGPD_Output, NAME_None, NAME_None, NAME_None));
}

bool URigMapperDefinitionEditorGraphNode::RemoveInputPin(UEdGraphPin* InPinToRemove)
{	
	bool bPinRemoved = RemovePin(InPinToRemove);
	if (bPinRemoved)
	{
		InputPins.Remove(InPinToRemove);
	}
	return bPinRemoved;
}

void URigMapperDefinitionEditorGraphNode::GetRect(FVector2D& TopLeft, FVector2D& BottomRight) const
{
	TopLeft.X = NodePosX;
	TopLeft.Y = NodePosY;
	BottomRight.X = NodePosX + GetDimensions().X + GetMargin().X;
	BottomRight.Y = NodePosY + GetDimensions().Y + GetMargin().Y;
}

void URigMapperDefinitionEditorGraphNode_Input::ApplyToDefinition(URigMapperDefinition* OutDefinition) const
{
	OutDefinition->Inputs.Add(NodeName);
}

void URigMapperDefinitionEditorGraphNode_WeightedSum::LoadFromFeature(const FRigMapperFeature* InFeature)
{
	if (InFeature && InFeature->GetFeatureType() == ERigMapperFeatureType::WeightedSum)
	{
		const FRigMapperWsFeature* WsFeature = static_cast<const FRigMapperWsFeature*>(InFeature);
		WeightedInputs.Reset();
		for (const TPair<FString, double>& Pair : WsFeature->Inputs)
		{
			WeightedInputs.Add(FRigMapperWsInput(Pair.Key, Pair.Value));
		}
		Range = WsFeature->Range;
	}
}

void URigMapperDefinitionEditorGraphNode_WeightedSum::ApplyToDefinition(URigMapperDefinition* OutDefinition) const
{
	FRigMapperWsFeature Feature(NodeName);
	Feature.Range = Range;

	// Rebuild the Inputs TMap from OrderedInputs
	Feature.Inputs.Empty();
	for (const FRigMapperWsInput& Input : WeightedInputs)
	{
		if (!Input.Name.IsEmpty())
		{
			Feature.Inputs.Add(Input.Name, Input.Weight);
		}
	}

	OutDefinition->Features.WeightedSums.Add(Feature);
}

void URigMapperDefinitionEditorGraphNode_WeightedSum::UpdateNodeFields()
{
	Super::UpdateNodeFields();
	const FText MinText = LOCTEXT("RangeMinimum", "min");
	const FText MaxText = LOCTEXT("RangeMaximum", "max");

	for (int32 PinIndex = 0; PinIndex < InputPins.Num(); PinIndex++)
	{
		if (WeightedInputs.IsValidIndex(PinIndex))
		{
			InputPins[PinIndex]->PinFriendlyName = FText::Format(INVTEXT("{0}"), WeightedInputs[PinIndex].Weight);
		}
	}
	FString RangeStr;
	if (Range.bHasLowerBound)
	{
		RangeStr.Appendf(TEXT("%s: %.*f"), *MinText.ToString(), 3, Range.LowerBound);
	}
	if (Range.bHasLowerBound && Range.bHasUpperBound)
	{
		RangeStr.Append(TEXT("\n"));
	}
	if (Range.bHasUpperBound)
	{
		RangeStr.Appendf(TEXT("%s: %.*f"), *MaxText.ToString(), 3, Range.UpperBound);
	}
	NodeSubtitle = FText::FromString(RangeStr);
}

FString URigMapperDefinitionEditorGraphNode_WeightedSum::GetNodeNameFromInputPin(int32 PinIndex) const
{
	if (WeightedInputs.IsValidIndex(PinIndex))
	{
		return WeightedInputs[PinIndex].Name;
	}
	return TEXT("");
}

void URigMapperDefinitionEditorGraphNode_SDK::LoadFromFeature(const FRigMapperFeature* InFeature)
{
	if (InFeature && InFeature->GetFeatureType() == ERigMapperFeatureType::SDK)
	{
		const FRigMapperSdkFeature* SdkFeature = static_cast<const FRigMapperSdkFeature*>(InFeature);
		Keys = SdkFeature->Keys;
	}
}

void URigMapperDefinitionEditorGraphNode_SDK::ApplyToDefinition(URigMapperDefinition* OutDefinition) const
{
	FRigMapperSdkFeature Feature(NodeName);
	Feature.Keys = Keys;

	// Get single input from pin connection
	const TArray<UEdGraphPin*>& InPins = GetInputPins();
	if (InPins.Num() > 0 && InPins[0]->LinkedTo.Num() > 0)
	{
		if (const URigMapperDefinitionEditorGraphNode* LinkedNode =
			Cast<URigMapperDefinitionEditorGraphNode>(InPins[0]->LinkedTo[0]->GetOwningNode()))
		{
			Feature.Input = LinkedNode->NodeName;
		}
	}

	OutDefinition->Features.SDKs.Add(Feature);
}

void URigMapperDefinitionEditorGraphNode_SDK::UpdateNodeFields()
{
	Super::UpdateNodeFields();
	TArray<FString> InKeys;
	TArray<FString> OutKeys;
	InKeys.Reserve(Keys.Num());
	OutKeys.Reserve(Keys.Num());
	for (const FRigMapperSdkKey& Key : Keys)
	{
		InKeys.Add(FString::Printf(TEXT("%.*f"), 3, Key.In));
		OutKeys.Add(FString::Printf(TEXT("%.*f"), 3, Key.Out));
	}
	const FString In = FString::Join(InKeys, TEXT(", "));
	const FString Out = FString::Join(OutKeys, TEXT(", "));
	NodeSubtitle = FText::FromString(FString::Printf(TEXT("[%s] > [%s]"), *In, *Out));
}

void URigMapperDefinitionEditorGraphNode_Multiply::ApplyToDefinition(URigMapperDefinition* OutDefinition) const
{
	FRigMapperMultiplyFeature Feature(NodeName);

	// Build input list from pin connections
	for (const UEdGraphPin* InPin : GetInputPins())
	{
		if (InPin->LinkedTo.Num() > 0)
		{
			if (const URigMapperDefinitionEditorGraphNode* LinkedNode =
				Cast<URigMapperDefinitionEditorGraphNode>(InPin->LinkedTo[0]->GetOwningNode()))
			{
				Feature.Inputs.Add(LinkedNode->NodeName);
			}
		}
	}

	OutDefinition->Features.Multiply.Add(Feature);
}

void URigMapperDefinitionEditorGraphNode_Multiply::UpdateNodeFields()
{
	Super::UpdateNodeFields();
}

void URigMapperDefinitionEditorGraphNode_MathOp::LoadFromFeature(const FRigMapperFeature* InFeature)
{
	if (InFeature && InFeature->GetFeatureType() == ERigMapperFeatureType::MathOp)
	{
		const FRigMapperMathFeature* MathFeature = static_cast<const FRigMapperMathFeature*>(InFeature);
		Operation = MathFeature->Operation;
		Inputs = MathFeature->Inputs;
	}
}

void URigMapperDefinitionEditorGraphNode_MathOp::ApplyToDefinition(URigMapperDefinition* OutDefinition) const
{
	FRigMapperMathFeature Feature(NodeName);
	Feature.Operation = Operation;
	Feature.Inputs = Inputs;
	OutDefinition->Features.MathOps.Add(Feature);
}

void URigMapperDefinitionEditorGraphNode_MathOp::UpdateNodeFields()
{
	Super::UpdateNodeFields();

	NodeSubtitle = UEnum::GetDisplayValueAsText(Operation);

	for (int32 PinIndex = 0; PinIndex < InputPins.Num(); ++PinIndex)
	{
		if (InputPins[PinIndex]->LinkedTo.Num() == 0 && Inputs.IsValidIndex(PinIndex))
		{
			InputPins[PinIndex]->PinFriendlyName = FText::Format(INVTEXT("{0}"), Inputs[PinIndex].ConstantValue);
		}
		else
		{
			InputPins[PinIndex]->PinFriendlyName = FText::GetEmpty();
		}
	}
}

int32 URigMapperDefinitionEditorGraphNode_MathOp::GetMinInputPinCount() const
{
	return FRigMapperMathFeature::GetMinInputCount(Operation);
}

FString URigMapperDefinitionEditorGraphNode_MathOp::GetNodeNameFromInputPin(int32 PinIndex) const
{
	if (Inputs.IsValidIndex(PinIndex))
	{
		return Inputs[PinIndex].NodeName;
	}
	return TEXT("");
}

void URigMapperDefinitionEditorGraphNode_Output::ApplyToDefinition(URigMapperDefinition* OutDefinition) const
{
	FString InputName;
	const TArray<UEdGraphPin*>& InPins = GetInputPins();
	if (InPins.Num() > 0 && InPins[0]->LinkedTo.Num() > 0)
	{
		if (const URigMapperDefinitionEditorGraphNode* LinkedNode =
			Cast<URigMapperDefinitionEditorGraphNode>(InPins[0]->LinkedTo[0]->GetOwningNode()))
		{
			InputName = LinkedNode->NodeName;
		}
	}
	OutDefinition->Outputs.Add(NodeName, InputName);
}

void URigMapperDefinitionEditorGraphNode_NullOutput::ApplyToDefinition(URigMapperDefinition* OutDefinition) const
{
	OutDefinition->NullOutputs.Add(NodeName);
}

void URigMapperCommentNode::OnRenameNode(const FString& NewName)
{
	Super::OnRenameNode(NewName);
	if (UEdGraph* Graph = GetGraph())
	{
		Graph->NotifyGraphChanged();
	}
}
#undef LOCTEXT_NAMESPACE

