// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraphUtilities.h"
#include "EdGraph/EdGraphNode.h"
#include "RigMapperDefinition.h"
#include "EdGraphNode_Comment.h"
#include "RigMapperDefinitionEditorGraphNode.generated.h"

#define UE_API RIGMAPPEREDITOR_API

/**
 * 
 */
UCLASS(MinimalAPI)
class URigMapperDefinitionEditorGraphNode : public UEdGraphNode
{
	GENERATED_BODY()

public:
	class NodeFactory : public FGraphPanelNodeFactory
	{
		virtual TSharedPtr<SGraphNode> CreateNode(UEdGraphNode* Node) const override;
	};

	UPROPERTY(EditAnywhere, Category = "Rig Mapper", meta = (DisplayPriority = 0))
	FString NodeName;

	// UEdGraphNode implementation
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override { return NodeTitle; }
	virtual FLinearColor GetNodeTitleColor() const override { return FLinearColor(0.969f, 0.5f, 0.466f); }
	virtual FLinearColor GetNodeBodyTintColor() const override { return GetNodeTitleColor(); }
	UE_API virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	UE_API virtual void AllocateDefaultPins() override;
	void PinConnectionListChanged(UEdGraphPin* Pin) override;
	UE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& Event) override;
	UE_API virtual void PostLoad() override;
	// !UEdGraphNode implementation

	virtual void LoadFromFeature(const FRigMapperFeature* InFeature) {}
	virtual void ApplyToDefinition(URigMapperDefinition* OutDefinition) const {}
	UE_API void SetupNode(const FString& InNodeName, ERigMapperFeatureType InNodeType);

	ERigMapperFeatureType GetNodeType() const { return NodeType; }

	const FText& GetSubtitle() const { return NodeSubtitle; }

	virtual int32 GetMinInputPinCount() const { return 0; }
	virtual int32 GetMaxInputPinCount() const { return MAX_int32; }  // unlimited by default
	virtual int32 GetInputPinCount() const { return InputPins.Num(); }
	virtual FString GetNodeNameFromInputPin(int32 PinIndex) const { return TEXT(""); };
	/** Updates node body */
	UE_API virtual void UpdateNodeFields();
	UE_API UEdGraphPin* CreateInputPin();
	UE_API UEdGraphPin* CreateOutputPin();
	UE_API bool RemoveInputPin(UEdGraphPin* InPinToRemove);
	const TArray<UEdGraphPin*>& GetInputPins() const { return InputPins; }
	UEdGraphPin* GetOutputPin() const { return OutputPins.IsEmpty() ? nullptr : OutputPins[0]; };

	void SetDimensions(const FVector2D& InDimensions) { Dimensions = InDimensions; }
	const FVector2D& GetDimensions() const { return Dimensions; }

	void SetMargin(const FVector2D& InDMargin) { Margin = InDMargin; }
	const FVector2D& GetMargin() const { return Margin; }

	UE_API void GetRect(FVector2D& TopLeft, FVector2D& BottomRight) const;

	UE_API void RelinkFeature(const FString& InNewInput, UEdGraphPin* InputPin);
	UE_API void AddNewInputLink(const FString& InNewInput);
	UE_API void RemoveLinksToFeature(const FString& InFeatureName);

protected:
	/** Cached title for the node */
	FText NodeTitle;

	/** Cached subtitle for the node */
	FText NodeSubtitle;

	/** Our input pins */
	TArray<UEdGraphPin*> InputPins;

	/** Our output pins */
	TArray<UEdGraphPin*> OutputPins;

	/** Cached dimensions of this node (used for layout) */
	FVector2D Dimensions = { 300, 50 };

	/** Cached dimensions of this node (used for layout) */
	FVector2D Margin = { 0, 0 };

	UPROPERTY()
	ERigMapperFeatureType NodeType;

	FString PreviousNodeName;
};

UCLASS(MinimalAPI)
class URigMapperDefinitionEditorGraphNode_Input : public URigMapperDefinitionEditorGraphNode
{
	GENERATED_BODY()

public:
	virtual FLinearColor GetNodeTitleColor() const override { return FLinearColor(0.466f, 0.969f, 0.878f); }

	virtual void ApplyToDefinition(URigMapperDefinition* OutDefinition) const override;
};

USTRUCT(BlueprintType)
struct FRigMapperWsInput
{
	GENERATED_BODY()

public:
	FRigMapperWsInput() {}

	explicit FRigMapperWsInput(const FString& InName, const double InWeight) : Name(InName), Weight(InWeight) {}

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rig Mapper")
	FString Name;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rig Mapper")
	double Weight = 0;
};

UCLASS(MinimalAPI)
class URigMapperDefinitionEditorGraphNode_WeightedSum : public URigMapperDefinitionEditorGraphNode
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Rig Mapper", meta = (DisplayPriority = 1))
	TArray<FRigMapperWsInput> WeightedInputs;

	UPROPERTY(EditAnywhere, Category = "Rig Mapper", meta = (DisplayPriority = 1))
	FRigMapperFeatureRange Range;

	virtual FLinearColor GetNodeTitleColor() const override { return FLinearColor(0.65f, 0.466f, 0.969f); }

	virtual void LoadFromFeature(const FRigMapperFeature* InFeature) override;

	virtual void ApplyToDefinition(URigMapperDefinition* OutDefinition) const override;

	UE_API virtual void UpdateNodeFields() override;

	virtual int32 GetMinInputPinCount() const { return 1; }

	virtual int32 GetInputPinCount() const override { return WeightedInputs.Num(); }

	virtual FString GetNodeNameFromInputPin(int32 PinIndex) const override;
};

UCLASS(MinimalAPI)
class URigMapperDefinitionEditorGraphNode_SDK : public URigMapperDefinitionEditorGraphNode
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Rig Mapper", meta = (DisplayPriority = 1))
	TArray<FRigMapperSdkKey> Keys;

	virtual FLinearColor GetNodeTitleColor() const override { return FLinearColor(0.969f, 0.717f, 0.466f); }

	virtual void LoadFromFeature(const FRigMapperFeature* InFeature) override;

	virtual void ApplyToDefinition(URigMapperDefinition* OutDefinition) const override;

	UE_API virtual void UpdateNodeFields() override;

};

UCLASS(MinimalAPI)
class URigMapperDefinitionEditorGraphNode_Multiply : public URigMapperDefinitionEditorGraphNode
{
	GENERATED_BODY()

public:
	virtual FLinearColor GetNodeTitleColor() const override { return FLinearColor(0.969f, 0.966f, 0.466f); }

	virtual void ApplyToDefinition(URigMapperDefinition* OutDefinition) const override;

	UE_API virtual void UpdateNodeFields() override;

	virtual int32 GetMinInputPinCount() const override { return 2; }
};

UCLASS(MinimalAPI)
class URigMapperDefinitionEditorGraphNode_MathOp : public URigMapperDefinitionEditorGraphNode
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Rig Mapper", meta = (DisplayPriority = 1))
	ERigMapperMathOperation Operation = ERigMapperMathOperation::Min;

	UPROPERTY(EditAnywhere, Category = "Rig Mapper", meta = (DisplayPriority = 2))
	TArray<FRigMapperMathInput> Inputs;

	virtual void LoadFromFeature(const FRigMapperFeature* InFeature) override;

	virtual void ApplyToDefinition(URigMapperDefinition* OutDefinition) const override;

	UE_API virtual void UpdateNodeFields() override;

	virtual FLinearColor GetNodeTitleColor() const override { return FLinearColor(0.969f, 0.466f, 0.65f); }

	UE_API virtual int32 GetMinInputPinCount() const override;

	virtual int32 GetInputPinCount() const override { return Inputs.Num(); }

	virtual FString GetNodeNameFromInputPin(int32 PinIndex) const override;
};

UCLASS(MinimalAPI)
class URigMapperDefinitionEditorGraphNode_Output : public URigMapperDefinitionEditorGraphNode
{
	GENERATED_BODY()

public:
	virtual FLinearColor GetNodeTitleColor() const override { return FLinearColor(0.466f, 0.969f, 0.525f); }

	virtual void ApplyToDefinition(URigMapperDefinition* OutDefinition) const override;
};

UCLASS(MinimalAPI)
class URigMapperDefinitionEditorGraphNode_NullOutput : public URigMapperDefinitionEditorGraphNode
{
	GENERATED_BODY()

public:
	virtual FLinearColor GetNodeTitleColor() const override { return FLinearColor(0.65f, 0.969f, 0.466f); }

	virtual void ApplyToDefinition(URigMapperDefinition* OutDefinition) const override;
};

UCLASS(MinimalAPI)
class URigMapperCommentNode : public UEdGraphNode_Comment
{
	GENERATED_BODY()
public:
	/** This is needed because CachedCommentTitle is never updated after OnRenameNode like in Blueprint graphs where the Blueprint modification chain triggers automatically. */
	virtual void OnRenameNode(const FString& NewName) override;
};

#undef UE_API
