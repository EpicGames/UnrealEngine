// Copyright Epic Games, Inc. All Rights Reserved.


#include "RigMapperDefinitionEditorGraphSchema.h"
#include "Framework/Commands/GenericCommands.h"
#include "GraphEditorActions.h"
#include "RigMapperDefinition.h"
#include "RigMapperDefinitionEditorGraphNode.h"
#include "RigMapperDefinitionEditorGraph.h"
#include "SRigMapperDefinitionGraphEditorNode.h"
#include "ToolMenu.h"
#include "ScopedTransaction.h"
#include "EdGraph/EdGraph.h"
#include "GraphEditor.h"
#include "SGraphPanel.h"
#include "SGraphNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigMapperDefinitionEditorGraphSchema)

#define LOCTEXT_NAMESPACE "RigMapperDefinitionEditorGraphSchema"

FSchemaAction_SpawnRigMapperNode::FSchemaAction_SpawnRigMapperNode() :
	FEdGraphSchemaAction(),
	NodeType(ERigMapperFeatureType::NullOutput)
{
}

FSchemaAction_SpawnRigMapperNode::FSchemaAction_SpawnRigMapperNode(ERigMapperFeatureType InNodeType, const FText& InCategory, const FText& InDescription, const FText& InTooltip, const int32 InGrouping) :
	FEdGraphSchemaAction(InCategory, InDescription, InTooltip, InGrouping),
	NodeType(InNodeType)
{
}

UEdGraphNode* FSchemaAction_SpawnRigMapperNode::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode /*= true*/)
{
	URigMapperDefinitionEditorGraphNode* NewNode = nullptr;
	if (URigMapperDefinitionEditorGraph* Graph = Cast<URigMapperDefinitionEditorGraph>(ParentGraph))
	{
		TSharedPtr<FScopedTransaction> Transaction = MakeShared<FScopedTransaction>(LOCTEXT("RigMapperDefinitionEditor_SpawnNewNode", "Spawn RigMapper definition node"));
		Graph->Modify();

		NewNode = Graph->CreateGraphNode(NodeType, FromPin, Location, bSelectNewNode);
		
		Transaction.Reset();
	}
	return NewNode;
}


UEdGraphNode* FSchemaAction_AddRigMapperComment::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
{
	TArray<UEdGraphNode*> SelectedNodes;

	// Grab the current selection from the graph editor hosting this graph
	if (TSharedPtr<SGraphEditor> GraphEditor = SGraphEditor::FindGraphEditorForGraph(ParentGraph))
	{
		const FGraphPanelSelectionSet& Selection = GraphEditor->GetSelectedNodes();
		SelectedNodes.Reserve(Selection.Num());
		for (UObject* Obj : Selection)
		{
			if (UEdGraphNode* Node = Cast<UEdGraphNode>(Obj))
			{
				SelectedNodes.Add(Node);
			}
		}
	}

	return URigMapperDefinitionEditorGraphSchema::CreateCommentNode(
		ParentGraph, Location, SelectedNodes, bSelectNewNode);
}


void URigMapperDefinitionEditorGraphSchema::GetContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	if (!Context->Pin && Context->Node && Context->Node->IsA<URigMapperDefinitionEditorGraphNode>())
	{
		// TODO: Implement context menu actions.
		FToolMenuSection& Section = Menu->AddSection("RigMapperDefinitionEditorNodeActions", LOCTEXT("RigMapperDefinitionEditor_NodeActionsMenuHeader", "Node Actions"));
		Section.AddMenuEntry(FGenericCommands::Get().Delete);
		Section.AddMenuEntry(FGraphEditorCommands::Get().BreakNodeLinks);
		// Focus Connected Nodes
		Section.AddMenuEntry(
			"FocusConnectedNodes",
			LOCTEXT("RigMapperDefinitionEditor_FocusConnectedNodes", "Focus Connected Nodes"),
			LOCTEXT("RigMapperDefinitionEditor_FocusConnectedNodesTooltip", "Show only this node and all connected nodes (F)"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([WeakNode = TWeakObjectPtr<const UEdGraphNode>(Context->Node)]()
				{
					if (const URigMapperDefinitionEditorGraphNode* NodePtr = Cast<URigMapperDefinitionEditorGraphNode>(WeakNode.Get()))
					{
						if (URigMapperDefinitionEditorGraph* Graph =
							Cast<URigMapperDefinitionEditorGraph>(NodePtr->GetGraph()))
						{
							Graph->OnFocusNodeRequested.ExecuteIfBound(const_cast<URigMapperDefinitionEditorGraphNode*>(NodePtr));
						}
					}
				}))
		);
	}
	Super::GetContextMenuActions(Menu, Context);	
}

FLinearColor URigMapperDefinitionEditorGraphSchema::GetPinTypeColor(const FEdGraphPinType& PinType) const
{
	return FLinearColor::White;
}

void URigMapperDefinitionEditorGraphSchema::BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotification) const
{
	Super::BreakPinLinks(TargetPin, bSendsNodeNotification);
}

void URigMapperDefinitionEditorGraphSchema::BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const
{
	Super::BreakSinglePinLink(SourcePin, TargetPin);
}

FPinConnectionResponse URigMapperDefinitionEditorGraphSchema::MovePinLinks(UEdGraphPin& MoveFromPin, UEdGraphPin& MoveToPin, bool bIsIntermediateMove, bool bNotifyLinkedNodes) const
{
	return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, TEXT(""));
}

FPinConnectionResponse URigMapperDefinitionEditorGraphSchema::CopyPinLinks(UEdGraphPin& CopyFromPin, UEdGraphPin& CopyToPin, bool bIsIntermediateCopy) const
{
	// Don't allow copying any links
	return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT(""));
}

FConnectionDrawingPolicy* URigMapperDefinitionEditorGraphSchema::CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const
{
	return new ConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, InZoomFactor, InClippingRect, InDrawElements);
}

FPinConnectionResponse URigMapperDefinitionEditorGraphSchema::CanCreateNewNodes(UEdGraphPin* InSourcePin) const
{
	return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, TEXT(""));
}

bool URigMapperDefinitionEditorGraphSchema::SupportsDropPinOnNode(UEdGraphNode* InTargetNode, const FEdGraphPinType& InSourcePinType, EEdGraphPinDirection InSourcePinDirection, FText& OutErrorMessage) const
{
	if (InSourcePinDirection == EGPD_Input)
	{
		OutErrorMessage = LOCTEXT("RigMapperDefinitionEditor_CannotCreateOutputConnection_DropPinOnNode", "It is not possible to create additional output connections on nodes.");
		return false;
	}

	if (const URigMapperDefinitionEditorGraphNode* GraphNode = Cast<URigMapperDefinitionEditorGraphNode>(InTargetNode))
	{
		if (GraphNode->GetNodeType() == ERigMapperFeatureType::Multiply 
			|| GraphNode->GetNodeType() == ERigMapperFeatureType::WeightedSum
			|| GraphNode->GetNodeType() == ERigMapperFeatureType::MathOp)
		{
			OutErrorMessage = LOCTEXT("RigMapperDefinitionEditor_CreateNewConnection_DropPinOnNode", "Create new input pin and connect.");
			return true;
		}
		OutErrorMessage = LOCTEXT("RigMapperDefinitionEditor_CannotCreateConnection_DropPinOnNode", "It is not possible to create new input connections on this node.");
	}
	return false;
}

void URigMapperDefinitionEditorGraphSchema::SetPinBeingDroppedOnNode(UEdGraphPin* InSourcePin) const
{
	PinBeingDroppedNodeName = TEXT("");
	if (!InSourcePin)
	{
		return;
	}
	if (URigMapperDefinitionEditorGraphNode* SourceNode = Cast<URigMapperDefinitionEditorGraphNode>(InSourcePin->GetOwningNode()))
	{
		PinBeingDroppedNodeName = SourceNode->NodeName;
	}
}

UEdGraphPin* URigMapperDefinitionEditorGraphSchema::DropPinOnNode(UEdGraphNode* InTargetNode, const FName& InSourcePinName, const FEdGraphPinType& InSourcePinType, EEdGraphPinDirection InSourcePinDirection) const
{
	if (PinBeingDroppedNodeName.IsEmpty())
	{
		return nullptr;
	}

	UEdGraphPin* NewPin = nullptr;
	if (InSourcePinDirection == EGPD_Output)
	{
		if (URigMapperDefinitionEditorGraphNode* GraphNode = Cast<URigMapperDefinitionEditorGraphNode>(InTargetNode))
		{
			NewPin = GraphNode->CreateInputPin();
			GraphNode->AddNewInputLink(PinBeingDroppedNodeName);
		}
	}
	PinBeingDroppedNodeName = TEXT("");
	return NewPin;
}

void URigMapperDefinitionEditorGraphSchema::GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const
{
	static const FText Category = LOCTEXT("RigMapperDefinitionEditor_AddNewGraphMenu", "Add new node");

	const UEdGraphPin* FromPin = ContextMenuBuilder.FromPin;

	if (!FromPin || FromPin->Direction == EGPD_Input)
	{
		ContextMenuBuilder.AddAction(MakeShared<FSchemaAction_SpawnRigMapperNode>(ERigMapperFeatureType::Input, Category,
			LOCTEXT("RigMapperDefinitionEditor_AddInputNode", "Add Input Node"),
			LOCTEXT("RigMapperDefinitionEditor_AddInputTooltip", "Spawn Input node (at the end of the input connector, if available)"),
			/*Grouping=*/0));
	}

	if (!FromPin || FromPin->Direction == EGPD_Output)
	{
		ContextMenuBuilder.AddAction(MakeShared<FSchemaAction_SpawnRigMapperNode>(ERigMapperFeatureType::Output, Category,
			LOCTEXT("RigMapperDefinitionEditor_AddOutputNode", "Add Output Node"),
			LOCTEXT("RigMapperDefinitionEditor_AddOutputNodeTooltip", "Spawn Output node (at the end of the output connector, if available)"),
			/*Grouping=*/0));
	}

	ContextMenuBuilder.AddAction(MakeShared<FSchemaAction_SpawnRigMapperNode>(ERigMapperFeatureType::Multiply, Category,
		LOCTEXT("RigMapperDefinitionEditor_AddMultiplyNode", "Add Multiply Node"),
		LOCTEXT("RigMapperDefinitionEditor_AddMultiplyNodeTooltip", "Spawn Multiply node (at the end of the connector, if available)"),
		/*Grouping=*/0));

	ContextMenuBuilder.AddAction(MakeShared<FSchemaAction_SpawnRigMapperNode>(ERigMapperFeatureType::SDK, Category,
		LOCTEXT("RigMapperDefinitionEditor_AddSDKNode", "Add SDK Node"),
		LOCTEXT("RigMapperDefinitionEditor_AddSDKNodeTooltip", "Spawn SDK node (at the end of the connector, if available)"),
		/*Grouping=*/0));

	ContextMenuBuilder.AddAction(MakeShared<FSchemaAction_SpawnRigMapperNode>(ERigMapperFeatureType::WeightedSum, Category,
		LOCTEXT("RigMapperDefinitionEditor_AddWeightedSumNode", "Add WeightedSum Node"),
		LOCTEXT("RigMapperDefinitionEditor_AddWeightedSumNodeTooltip", "Spawn WeightedSum node (at the end of the connector, if available)"),
		/*Grouping=*/0));

	ContextMenuBuilder.AddAction(MakeShared<FSchemaAction_SpawnRigMapperNode>(ERigMapperFeatureType::MathOp, Category,
		LOCTEXT("RigMapperDefinitionEditor_AddMathOpNode", "Add MathOp Node"),
		LOCTEXT("RigMapperDefinitionEditor_AddMathOpNodeTooltip", "Spawn Math Operation node"),
		/*Grouping=*/0));

	if (!FromPin)
	{
		ContextMenuBuilder.AddAction(MakeShared<FSchemaAction_SpawnRigMapperNode>(ERigMapperFeatureType::NullOutput, Category,
			LOCTEXT("RigMapperDefinitionEditor_AddNullOutputNode", "Add Null Output Node"),
			LOCTEXT("RigMapperDefinitionEditor_AddNullOutputNodeTooltip", "Spawn null output node"),
			/*Grouping=*/0));

		// Add Comment
		ContextMenuBuilder.AddAction(GetCreateCommentAction());
	}
}

const FPinConnectionResponse URigMapperDefinitionEditorGraphSchema::CanCreateConnection(const UEdGraphPin* InA, const UEdGraphPin* InB) const
{
	// Make sure the pins are not on the same node
	if (InA->GetOwningNode() == InB->GetOwningNode())
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("RigMapperDefinitionEditor_ConnectionSameNode", "Both are on the same node"));
	}

	// Compare the directions
	const UEdGraphPin* InputPin = NULL;
	const UEdGraphPin* OutputPin = NULL;

	if (!CategorizePinsByDirection(InA, InB, /*out*/ InputPin, /*out*/ OutputPin))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("RigMapperDefinitionEditor_ConnectionIncompatible", "Directions are not compatible"));
	}

	// Check for new and existing loops
	FText ResponseMessage;
	//if (ConnectionCausesLoop(InputPin, OutputPin))
	//{
	//	ResponseMessage = LOCTEXT("ConnectionLoop", "Connection could cause loop");
	//}

	// For non-exec pins, break existing connections on inputs only - multiple output connections are acceptable
	if (InputPin->LinkedTo.Num() > 0)
	{
		ECanCreateConnectionResponse ReplyBreakOutputs;
		if (InputPin == InA)
		{
			ReplyBreakOutputs = CONNECT_RESPONSE_BREAK_OTHERS_A;
		}
		else
		{
			ReplyBreakOutputs = CONNECT_RESPONSE_BREAK_OTHERS_B;
		}
		if (ResponseMessage.IsEmpty())
		{
			ResponseMessage = LOCTEXT("RigMapperDefinitionEditor_ConnectionReplace", "Replace existing connections");
		}
		return FPinConnectionResponse(ReplyBreakOutputs, ResponseMessage);
	}

	return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, TEXT(""));
}

bool URigMapperDefinitionEditorGraphSchema::TryCreateConnection(UEdGraphPin* InA, UEdGraphPin* InB) const
{
	if (InA == InB)
	{
		return false;
	}
	if (InA->GetOwningNode() == InB->GetOwningNode())
	{
		return false;
	}

	bool bModified = false;
	FPinConnectionResponse ConnResponse = CanCreateConnection(InA, InB);
	if (ConnResponse.Response == CONNECT_RESPONSE_DISALLOW)
	{
		return false;
	}

	// If there is an input pin with an existing connection, it is about to be broken.
	// We need to save the name of that node in order to update the Definition.
	UEdGraphPin* InputPin = nullptr;
	const UEdGraphPin* OutputPin = nullptr;
	FString NewInputName = TEXT("");
	if (ConnResponse.Response == CONNECT_RESPONSE_BREAK_OTHERS_A)
	{
		// Pin A is an input pin with existing connection which is about to be broken.
		InputPin = InA;
		OutputPin = InB;
	}
	else if (ConnResponse.Response == CONNECT_RESPONSE_BREAK_OTHERS_B)
	{
		// Pin B is an input pin with existing connection which is about to be broken.
		InputPin = InB;
		OutputPin = InA;
	}
	else if (InA->Direction == EGPD_Input)
	{
		InputPin = InA;
		OutputPin = InB;
	}
	else
	{
		InputPin = InB;
		OutputPin = InA;
	}
	if (OutputPin)
	{
		if (URigMapperDefinitionEditorGraphNode* NewNode = Cast<URigMapperDefinitionEditorGraphNode>(OutputPin->GetOwningNode()))
		{
			NewInputName = NewNode->NodeName;
		}
	}
	bModified = Super::TryCreateConnection(InA, InB);
	
	if (bModified)
	{
		URigMapperDefinitionEditorGraphNode* InputNode = Cast<URigMapperDefinitionEditorGraphNode>(InputPin->GetOwningNode());
		InputNode->RelinkFeature(NewInputName, InputPin);
	}

	return bModified;
}

bool URigMapperDefinitionEditorGraphSchema::IsConnectionRelinkingAllowed(UEdGraphPin* InPin) const
{
	return true;
}

const FPinConnectionResponse URigMapperDefinitionEditorGraphSchema::CanRelinkConnectionToPin(const UEdGraphPin* OldSourcePin, const UEdGraphPin* TargetPinCandidate) const
{
	return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, TEXT(""));
}

bool URigMapperDefinitionEditorGraphSchema::CreatePromotedConnection(UEdGraphPin* A, UEdGraphPin* B) const
{
	return Super::CreatePromotedConnection(A, B);
}

bool URigMapperDefinitionEditorGraphSchema::TryRelinkConnectionTarget(UEdGraphPin* SourcePin, UEdGraphPin* OldTargetPin, UEdGraphPin* NewTargetPin, const TArray<UEdGraphNode*>& InSelectedGraphNodes) const
{
	return Super::TryRelinkConnectionTarget(SourcePin, OldTargetPin, NewTargetPin, InSelectedGraphNodes);
}

bool URigMapperDefinitionEditorGraphSchema::SafeDeleteNodeFromGraph(UEdGraph* Graph, UEdGraphNode* Node) const
{
	if (!Graph)
	{
		return false;
	}

	if (!Node)
	{
		return false;
	}

	if (URigMapperDefinitionEditorGraphNode* RigMapperNode = Cast<URigMapperDefinitionEditorGraphNode>(Node))
	{
		if (const UEdGraphPin* OutputPin = RigMapperNode->GetOutputPin())
		{
			for (const UEdGraphPin* LinkedPin : OutputPin->LinkedTo)
			{
				if (URigMapperDefinitionEditorGraphNode* LinkedNode = Cast<URigMapperDefinitionEditorGraphNode>(LinkedPin->GetOwningNode()))
				{
					LinkedNode->RemoveLinksToFeature(RigMapperNode->NodeName);
				}
			}
		}

		if (URigMapperDefinitionEditorGraph* RigMapperGraph = Cast<URigMapperDefinitionEditorGraph>(Graph))
		{
			RigMapperGraph->RemoveNodeFromMap(RigMapperNode->NodeName, RigMapperNode->GetNodeType());
		}


		return true;		
	}
	if (URigMapperCommentNode* CommentNode = Cast<URigMapperCommentNode>(Node))
	{
		// Comment nodes are not kept in the internal node map.
		return true;
	}
	return false;
}

TSharedPtr<FEdGraphSchemaAction> URigMapperDefinitionEditorGraphSchema::GetCreateCommentAction() const
{
	return MakeShared<FSchemaAction_AddRigMapperComment>(
		FText(),
		LOCTEXT("RigMapperDefinitionEditor_AddComment", "Add Comment"),
		LOCTEXT("RigMapperDefinitionEditor_AddCommentTooltip", "Add a comment box"));
}

URigMapperCommentNode* URigMapperDefinitionEditorGraphSchema::CreateCommentNode(UEdGraph* InParentGraph, const FVector2f& InLocation, const TArray<UEdGraphNode*>& InSelectedNodes, bool bInSelectNewNode)
{
	if (!InParentGraph)
	{
		return nullptr;
	}

	FScopedTransaction Transaction(LOCTEXT("RigMapperDefinitionEditor_SpawnNewCommentNode", "Spawn comment node"));
	InParentGraph->Modify();

	URigMapperCommentNode* CommentNode = NewObject<URigMapperCommentNode>(InParentGraph, NAME_None, RF_Transactional);
	InParentGraph->AddNode(CommentNode, /*bUserAction=*/true, bInSelectNewNode);
	CommentNode->CreateNewGuid();
	CommentNode->PostPlacedNewNode();

	// Filter out comment nodes from bounds calculation
	TArray<UEdGraphNode*> NodesToEnclose;
	NodesToEnclose.Reserve(InSelectedNodes.Num());
	for (UEdGraphNode* Node : InSelectedNodes)
	{
		if (Node && !Node->IsA<URigMapperCommentNode>())
		{
			NodesToEnclose.Add(Node);
		}
	}

	if (NodesToEnclose.IsEmpty())
	{
		CommentNode->NodePosX = static_cast<int32>(InLocation.X);
		CommentNode->NodePosY = static_cast<int32>(InLocation.Y);
	}
	else
	{
		// Get the live graph panel so we can query actual rendered widget sizes
		SGraphPanel* GraphPanel = nullptr;
		if (TSharedPtr<SGraphEditor> GraphEditor = SGraphEditor::FindGraphEditorForGraph(InParentGraph))
		{
			GraphPanel = GraphEditor->GetGraphPanel();
		}

		FSlateRect Bounds(FLT_MAX, FLT_MAX, -FLT_MAX, -FLT_MAX);
		for (UEdGraphNode* Node : NodesToEnclose)
		{
			const float NodeX = static_cast<float>(Node->NodePosX);
			const float NodeY = static_cast<float>(Node->NodePosY);

			// Prefer the actual rendered size from the Slate widget; fall back to the node's stored size
			FVector2D NodeSize(
				FMath::Max(Node->NodeWidth, 100),
				FMath::Max(Node->NodeHeight, 50));

			if (GraphPanel)
			{
				if (TSharedPtr<SGraphNode> NodeWidget = GraphPanel->GetNodeWidgetFromGuid(Node->NodeGuid))
				{
					const FVector2D DesiredSize = NodeWidget->GetDesiredSize();
					if (DesiredSize.X > 0.0f && DesiredSize.Y > 0.0f)
					{
						NodeSize = DesiredSize;
					}
				}
			}

			Bounds.Left = FMath::Min(Bounds.Left, NodeX);
			Bounds.Top = FMath::Min(Bounds.Top, NodeY);
			Bounds.Right = FMath::Max(Bounds.Right, NodeX + static_cast<float>(NodeSize.X));
			Bounds.Bottom = FMath::Max(Bounds.Bottom, NodeY + static_cast<float>(NodeSize.Y));
		}

		constexpr float HorizontalPadding = 30.0f;
		constexpr float VerticalPadding = 30.0f;
		constexpr float TitleBarHeight = 35.0f;

		CommentNode->NodePosX = static_cast<int32>(Bounds.Left - HorizontalPadding);
		CommentNode->NodePosY = static_cast<int32>(Bounds.Top - VerticalPadding - TitleBarHeight);
		CommentNode->NodeWidth = static_cast<int32>((Bounds.Right - Bounds.Left) + HorizontalPadding * 2.0f);
		CommentNode->NodeHeight = static_cast<int32>((Bounds.Bottom - Bounds.Top) + VerticalPadding * 2.0f + TitleBarHeight);
	}

	return CommentNode;
}

#undef LOCTEXT_NAMESPACE