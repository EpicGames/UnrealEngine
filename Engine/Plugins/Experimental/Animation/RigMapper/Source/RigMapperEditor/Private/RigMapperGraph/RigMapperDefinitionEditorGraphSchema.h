// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConnectionDrawingPolicy.h"
#include "EdGraph/EdGraphSchema.h"

#include "RigMapperDefinitionEditorGraphSchema.generated.h"

#define UE_API RIGMAPPEREDITOR_API

enum class ERigMapperFeatureType : uint8;
class URigMapperCommentNode;

USTRUCT()
struct FSchemaAction_SpawnRigMapperNode : public FEdGraphSchemaAction
{
	GENERATED_BODY()

public:
	FSchemaAction_SpawnRigMapperNode();

	FSchemaAction_SpawnRigMapperNode(ERigMapperFeatureType InNodeType, const FText& InCategory, const FText& InDescription, const FText& InTooltip, const int InGrouping);

	//~ Begin FEdGraphSchemaAction Interface
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface
private:
	ERigMapperFeatureType NodeType;
};

/** Action to add a 'comment' node to the graph */
USTRUCT()
struct FSchemaAction_AddRigMapperComment : public FEdGraphSchemaAction
{
	GENERATED_BODY()

public:
	FSchemaAction_AddRigMapperComment() {}
	FSchemaAction_AddRigMapperComment(const FText& InCategory, const FText& InDescription, const FText& InTooltip)
		: FEdGraphSchemaAction(InCategory, InDescription, InTooltip, 0) {}

	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
};

/**
 * 
 */
UCLASS(MinimalAPI)
class URigMapperDefinitionEditorGraphSchema : public UEdGraphSchema
{
	GENERATED_BODY()

public:
	class ConnectionDrawingPolicy : public FConnectionDrawingPolicy
	{
	public:
		ConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements)
			: FConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, InZoomFactor, InClippingRect, InDrawElements)
		{
			// We don't draw arrows here
			ArrowImage = nullptr;
			ArrowRadius = FVector2D(0.0f, 0.0f);
		}

		virtual FVector2f ComputeSplineTangent(const FVector2f& Start, const FVector2f& End) const override
		{
			const int32 Tension = FMath::Abs<int32>(Start.X - End.X);
			return Tension * FVector2f(1.0f, 0);
		}
	};
	
	// UEdGraphSchema interface
	UE_API virtual void GetContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;	
	virtual FName GetParentContextMenuName() const override { return NAME_None; }
	UE_API virtual FLinearColor GetPinTypeColor(const FEdGraphPinType& PinType) const override;
	UE_API virtual void BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotification) const override;
	UE_API virtual void BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const override;
	UE_API virtual FPinConnectionResponse MovePinLinks(UEdGraphPin& MoveFromPin, UEdGraphPin& MoveToPin, bool bIsIntermediateMove = false, bool bNotifyLinkedNodes = false) const override;
	UE_API virtual FPinConnectionResponse CopyPinLinks(UEdGraphPin& CopyFromPin, UEdGraphPin& CopyToPin, bool bIsIntermediateCopy = false) const override;
	UE_API virtual class FConnectionDrawingPolicy* CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const override;
	virtual bool ShouldAlwaysPurgeOnModification() const override { return false; }
	UE_API virtual FPinConnectionResponse CanCreateNewNodes(UEdGraphPin* InSourcePin) const override;
	UE_API virtual bool SupportsDropPinOnNode(UEdGraphNode* InTargetNode, const FEdGraphPinType& InSourcePinType, EEdGraphPinDirection InSourcePinDirection, FText& OutErrorMessage) const override;
	UE_API void GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const override;
	UE_API virtual const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* InA, const UEdGraphPin* InB) const override;
	UE_API bool TryCreateConnection(UEdGraphPin* InA, UEdGraphPin* InB) const override;
	UE_API bool IsConnectionRelinkingAllowed(UEdGraphPin* InPin) const override;
	UE_API const FPinConnectionResponse CanRelinkConnectionToPin(const UEdGraphPin* OldSourcePin, const UEdGraphPin* TargetPinCandidate) const override;
	UE_API bool CreatePromotedConnection(UEdGraphPin* InA, UEdGraphPin* InB) const override;
	UEdGraphPin* DropPinOnNode(UEdGraphNode* InTargetNode, const FName& InSourcePinName, const FEdGraphPinType& InSourcePinType, EEdGraphPinDirection InSourcePinDirection) const override;
	void SetPinBeingDroppedOnNode(UEdGraphPin* InSourcePin) const override;
	UE_API bool TryRelinkConnectionTarget(UEdGraphPin* SourcePin, UEdGraphPin* OldTargetPin, UEdGraphPin* NewTargetPin, const TArray<UEdGraphNode*>& InSelectedGraphNodes) const override;
	bool SafeDeleteNodeFromGraph(UEdGraph* Graph, UEdGraphNode* Node) const override;
	UE_API virtual TSharedPtr<FEdGraphSchemaAction> GetCreateCommentAction() const override;
	// End of UEdGraphSchema interface

	/**
	 * Creates a comment node on the graph. 
	 * If SelectedNodes is not empty, sizes the comment to enclose them and ignores Location. 
	 * Otherwise places at Location with default size.
	 */
	static UE_API URigMapperCommentNode* CreateCommentNode(
		UEdGraph* InParentGraph,
		const FVector2f& InLocation,
		const TArray<UEdGraphNode*>& InSelectedNodes,
		bool bInSelectNewNode = true);

private:
	/** The name of the owning node of the pin that is being dropped on another node during a drag-drop operation. Controlled by SupportsDropPinOnNode(). */
	mutable FString PinBeingDroppedNodeName = TEXT("");
};

#undef UE_API
