// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigMapperGraphCommands.h"

#define LOCTEXT_NAMESPACE "RigMapperGraphCommands"

void FRigMapperGraphCommands::RegisterCommands()
{
	UI_COMMAND(FocusConnectedNodes, "Focus Connected Nodes",
		"Show only the selected node and all nodes connected to it",
		EUserInterfaceActionType::Button, FInputChord(EKeys::F));

	UI_COMMAND(CreateComment, "Create Comment",
		"Create a comment node enclosing the selected nodes",
		EUserInterfaceActionType::Button, FInputChord(EKeys::C));
}

#undef LOCTEXT_NAMESPACE

