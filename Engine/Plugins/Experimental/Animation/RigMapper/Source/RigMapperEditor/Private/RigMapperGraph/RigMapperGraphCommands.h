// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

class FRigMapperGraphCommands : public TCommands<FRigMapperGraphCommands>
{
public:
	FRigMapperGraphCommands()
		: TCommands<FRigMapperGraphCommands>(
			TEXT("RigMapperGraph"),
			NSLOCTEXT("RigMapperDefinitionGraphCommands", "RigMapperDefinitionGraphCommands", "Rig Mapper Graph commands"),
			NAME_None,
			FAppStyle::GetAppStyleSetName()) {
	}

	TSharedPtr<FUICommandInfo> FocusConnectedNodes;
	TSharedPtr<FUICommandInfo> CreateComment;

	virtual void RegisterCommands() override;
};