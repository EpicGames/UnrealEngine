// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRigMapperDefinitionGraphEditor.h"

#include "SRigMapperValidationBanner.h"
#include "RigMapperDefinitionEditorGraph.h"
#include "RigMapperDefinitionEditorGraphNode.h"
#include "RigMapperDefinitionEditorGraphSchema.h"
#include "RigMapperGraphCommands.h"
#include "RigMapperDefinition.h"
#include "ScopedTransaction.h"

#include "SGraphPanel.h"
#include "SlateOptMacros.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Application/SlateApplication.h"
#include "Layout/WidgetPath.h"

#if WITH_EDITOR
#include "Editor/EditorEngine.h"
extern UNREALED_API UEditorEngine* GEditor;
#endif
BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

#define LOCTEXT_NAMESPACE "SRigMapperDefinitionGraphEditor"

SRigMapperDefinitionGraphEditor::~SRigMapperDefinitionGraphEditor()
{
	if (!GExitPurge)
	{
		if (ensure(GraphObj))
		{
			GraphObj->RemoveFromRoot();
		}		
	}
}

void SRigMapperDefinitionGraphEditor::Construct(const FArguments& InArgs, URigMapperDefinition* InDefinition)
{
	check(InDefinition);

	FGraphAppearanceInfo AppearanceInfo;
	AppearanceInfo.CornerText = LOCTEXT("GraphEditorRigMapperDefinition", "Rig Mapper Definition");

	SGraphEditor::FGraphEditorEvents GraphEvents;
	// GraphEvents.OnCreateActionMenu = SGraphEditor::FOnCreateActionMenu::CreateSP(this, &SPhysicsAssetGraph::OnCreateGraphActionMenu);
	GraphEvents.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateSP(this, &SRigMapperDefinitionGraphEditor::HandleSelectionChanged);
	GraphEvents.OnTextCommitted = FOnNodeTextCommitted::CreateLambda(
		[](const FText& InText, ETextCommit::Type CommitInfo, UEdGraphNode* InNode)
		{
			if (InNode && CommitInfo != ETextCommit::OnCleared)
			{
				InNode->OnRenameNode(InText.ToString());
			}
		});
	// GraphEvents.OnNodeDoubleClicked = FSingleNodeEvent::CreateSP(this, &SPhysicsAssetGraph::HandleNodeDoubleClicked);	

	GraphObj = Cast<URigMapperDefinitionEditorGraph>(InDefinition->EditorGraph);
	check(GraphObj);
	GraphObj->AddToRoot();
	GraphObj->OnFocusNodeRequested.BindSP(this, &SRigMapperDefinitionGraphEditor::SetFocusOnNode);
	BindCommands();

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot().AutoHeight()
		[
			SAssignNew(ValidationBanner, SRigMapperValidationBanner)
		]
		+SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			SAssignNew(GraphEditor, SGraphEditor)
			.GraphToEdit(GraphObj)
			.GraphEvents(GraphEvents)
			.Appearance(AppearanceInfo)
			.ShowGraphStateOverlay(false)
			.IsEditable(true)
			.AutoExpandActionMenu(true)
			.AdditionalCommands(UICommandList)
			.DisplayAsReadOnly(false)				
		]
	];
}

void SRigMapperDefinitionGraphEditor::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (GraphObj->NeedsRefreshLayout())
	{		
		GraphObj->LayoutNodes();

		GraphObj->RequestRefreshLayout(false);

		//const FGraphPanelSelectionSet& Selection = GraphEditor->GetSelectedNodes();

		//TArray<URigMapperDefinitionEditorGraphNode*> SelectedNodes;
		//SelectedNodes.Reserve(Selection.Num());

		//for (UObject* Obj : Selection)
		//{
		//	if (URigMapperDefinitionEditorGraphNode* Node = Cast<URigMapperDefinitionEditorGraphNode>(Obj))
		//	{
		//		SelectedNodes.Add(Node);
		//	}
		//}

		//ZoomToFitNodes(SelectedNodes);
	}
}

void SRigMapperDefinitionGraphEditor::SelectNodes(const TArray<FString>& Inputs, const TArray<FString>& Features, const TArray<FString>& Outputs, const TArray<FString>& NullOutputs)
{
	if (!bSelectingNodes)
	{
		bSelectingNodes = true;
		
		GraphEditor->ClearSelectionSet();

		const TArray<URigMapperDefinitionEditorGraphNode*>& SelectedNodes = GraphObj->GetNodesByName(Inputs, Features, Outputs, NullOutputs); 
		
		for (URigMapperDefinitionEditorGraphNode* Node : SelectedNodes)
		{
			GraphEditor->SetNodeSelection(Node, true);
		}
		ZoomToFitNodes(SelectedNodes);
		
		bSelectingNodes = false;
	}
}

void SRigMapperDefinitionGraphEditor::RebuildGraph()
{
	GraphObj->RebuildGraph();
}

void SRigMapperDefinitionGraphEditor::RefreshGraphNode(URigMapperDefinitionEditorGraphNode* InNode)
{
	if (InNode)
	{
		InNode->Modify();
		GraphEditor->RefreshNode(*InNode);
	}
}

void SRigMapperDefinitionGraphEditor::CaptureKeyboard()
{
#if WITH_EDITOR
	FTimerHandle TimerHandle;
	GEditor->GetTimerManager()->SetTimer(TimerHandle, [WeakGraphEditor = GraphEditor.ToWeakPtr()]()
		{
			if (WeakGraphEditor.IsValid())
			{
				SGraphPanel* GraphPanel = WeakGraphEditor.Pin()->GetGraphPanel();
				if (GraphPanel)
				{
					FWidgetPath SlatePath;
					if (FSlateApplication::Get().GeneratePathToWidgetUnchecked(GraphPanel->AsShared(), SlatePath))
					{
						FSlateApplication::Get().SetKeyboardFocus(SlatePath, EFocusCause::SetDirectly);
					}
					else
					{
						WeakGraphEditor.Pin()->CaptureKeyboard();
					}
				}
			}
		}, 0.01f, false);
#endif
}

void SRigMapperDefinitionGraphEditor::ZoomToFitNodes(const TArray<URigMapperDefinitionEditorGraphNode*>& SelectedNodes) const
{
	if (GraphObj->NeedsRefreshLayout())
	{
		return;
	}
	if (!bFocusLinkedNodes)
	{
		GraphEditor->ZoomToFit(true);
	}
	else
	{
		if (!SelectedNodes.IsEmpty() && !GraphEditor->GetGraphPanel()->HasDeferredZoomDestination())
		{
			FVector2D MinCorner(MAX_FLT, MAX_FLT);
			FVector2D MaxCorner(-MAX_FLT, -MAX_FLT);

			const FVector2D MaxLinkedNodeOffset(600, 400);
			
			for (UObject* Obj : SelectedNodes)
			{
				URigMapperDefinitionEditorGraphNode* Node = Cast<URigMapperDefinitionEditorGraphNode>(Obj);

				FVector2D NodeTopLeft;
				FVector2D NodeBottomRight;
				Node->GetRect(NodeTopLeft, NodeBottomRight);
				
				TArray<URigMapperDefinitionEditorGraphNode*> LinkedNodes = { Node };
				GetAllLinkedNodes(Node, LinkedNodes, true);
				GetAllLinkedNodes(Node, LinkedNodes, false);

				for (URigMapperDefinitionEditorGraphNode* LinkedNode : LinkedNodes)
				{
					FVector2D LinkedNodeTopLeft;
					FVector2D LinkedNodeBottomRight;
					LinkedNode->GetRect(LinkedNodeTopLeft, LinkedNodeBottomRight);
					
					MinCorner.X = FMath::Min(MinCorner.X, FMath::Max(LinkedNodeTopLeft.X, NodeTopLeft.X - MaxLinkedNodeOffset.X));
					MinCorner.Y = FMath::Min(MinCorner.Y, FMath::Max(LinkedNodeTopLeft.Y, NodeTopLeft.Y - MaxLinkedNodeOffset.Y));
					MaxCorner.X = FMath::Max(MaxCorner.X, FMath::Min(LinkedNodeBottomRight.X, NodeBottomRight.X + MaxLinkedNodeOffset.X));
					MaxCorner.Y = FMath::Max(MaxCorner.Y, FMath::Min(LinkedNodeBottomRight.Y, NodeBottomRight.Y + MaxLinkedNodeOffset.Y));			
				}
			}
			GraphEditor->GetGraphPanel()->JumpToRect(MinCorner, MaxCorner);
		}
	}
}

void SRigMapperDefinitionGraphEditor::GetAllLinkedNodes(const URigMapperDefinitionEditorGraphNode* BaseNode, TArray<URigMapperDefinitionEditorGraphNode*>& LinkedNodes, bool bDescend)
{
	
	TArray<UEdGraphPin*> Pins;
	if (bDescend)
	{
		Pins = BaseNode->GetInputPins();
	}
	else
	{
		Pins = { BaseNode->GetOutputPin() };
	}
	
	for (const UEdGraphPin* PinA : Pins)
	{
		if (PinA)
		{
			for (const UEdGraphPin* PinB : PinA->LinkedTo)
			{
				if (PinB)
				{
					URigMapperDefinitionEditorGraphNode* LinkedNode = Cast<URigMapperDefinitionEditorGraphNode>(PinB->GetOwningNode());

					if (LinkedNode && !LinkedNodes.Contains(LinkedNode))
					{
						LinkedNodes.Add(LinkedNode);
						GetAllLinkedNodes(LinkedNode, LinkedNodes, bDescend);
					}
				}
			}
		}
	}
}

void SRigMapperDefinitionGraphEditor::HandleSelectionChanged(const TSet<UObject*>& Nodes)
{
	if (!bSelectingNodes)
	{
		bSelectingNodes = true;
		
		if (OnSelectionChanged.IsBound())
		{
			OnSelectionChanged.Execute(Nodes);
		}
		
		bSelectingNodes = false;
	}
}

FReply SRigMapperDefinitionGraphEditor::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	return UICommandList->ProcessCommandBindings(InKeyEvent) ? FReply::Handled() : FReply::Unhandled();
}

void SRigMapperDefinitionGraphEditor::BindCommands()
{
	// This should not be called twice on the same instance
	check(!UICommandList.IsValid());

	UICommandList = MakeShareable(new FUICommandList);

	FUICommandList& CommandList = *UICommandList;

	// ...and bind them all
	CommandList.MapAction(
		FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &SRigMapperDefinitionGraphEditor::OnDeleteSelectedNodes),
		FCanExecuteAction::CreateSP(this, &SRigMapperDefinitionGraphEditor::CanDeleteNodes));

	CommandList.MapAction(
		FRigMapperGraphCommands::Get().FocusConnectedNodes,
		FExecuteAction::CreateSP(this, &SRigMapperDefinitionGraphEditor::OnToggleFocusMode),
		FCanExecuteAction());

	CommandList.MapAction(
		FRigMapperGraphCommands::Get().CreateComment,
		FExecuteAction::CreateSP(this, &SRigMapperDefinitionGraphEditor::OnCreateComment),
		FCanExecuteAction());
}

void SRigMapperDefinitionGraphEditor::OnDeleteSelectedNodes()
{
	UEdGraph* CurrentGraph = GraphEditor->GetCurrentGraph();
	if (!CurrentGraph)
	{
		return;
	}

	const UEdGraphSchema* Schema = CurrentGraph->GetSchema();	
	const FGraphPanelSelectionSet& Selection = GraphEditor->GetSelectedNodes();
	TSharedPtr<FScopedTransaction> Transaction = MakeShared<FScopedTransaction>(FGenericCommands::Get().Delete->GetDescription());

	TArray<UEdGraphNode*> NodesToDelete;
	NodesToDelete.Reserve(Selection.Num());

	for (UObject* Obj : Selection)
	{
		if (UEdGraphNode* Node = Cast<UEdGraphNode>(Obj))
		{
			NodesToDelete.Add(Node);
		}
	}

	CurrentGraph->Modify();
	for (UEdGraphNode* Node : NodesToDelete)
	{
		if (Schema)
		{
			Schema->SafeDeleteNodeFromGraph(CurrentGraph, Node);
		}
		Node->Modify();
		Node->DestroyNode();
	}
	GraphEditor->ClearSelectionSet();

	if (URigMapperDefinitionEditorGraph* RigMapperGraph = Cast<URigMapperDefinitionEditorGraph>(CurrentGraph))
	{
		RigMapperGraph->OnGraphStructureUpdated.Broadcast();   // Refresh the structure view.
	}
}

bool SRigMapperDefinitionGraphEditor::CanDeleteNodes()
{
	return true;
}

void SRigMapperDefinitionGraphEditor::SetFocusOnNode(URigMapperDefinitionEditorGraphNode* InNode)
{
	bFocusMode = true;
	FocusedNodes.Reset();
	GraphEditor->ClearSelectionSet();

	// Collect selected node and nodes connected to it
	FocusedNodes.Add(InNode);
	TArray<URigMapperDefinitionEditorGraphNode*> LinkedNodes = { InNode }; // starting node
	GetAllLinkedNodes(InNode, LinkedNodes, true);   // downstream
	GetAllLinkedNodes(InNode, LinkedNodes, false);  // upstream
	for (URigMapperDefinitionEditorGraphNode* LinkedNode : LinkedNodes)
	{
		FocusedNodes.Add(LinkedNode);
		GraphEditor->SetNodeSelection(LinkedNode, true);
	}

	// Zoom on selected nodes
	if (!LinkedNodes.IsEmpty() && !GraphEditor->GetGraphPanel()->HasDeferredZoomDestination())
	{
		FVector2D MinCorner(MAX_FLT, MAX_FLT);
		FVector2D MaxCorner(-MAX_FLT, -MAX_FLT);
		FVector2D NodeTopLeft;
		FVector2D NodeBottomRight;
		for (URigMapperDefinitionEditorGraphNode* LinkedNode : LinkedNodes)
		{
			LinkedNode->GetRect(NodeTopLeft, NodeBottomRight);

			MinCorner.X = FMath::Min(MinCorner.X, NodeTopLeft.X);
			MinCorner.Y = FMath::Min(MinCorner.Y, NodeTopLeft.Y);
			MaxCorner.X = FMath::Max(MaxCorner.X, NodeBottomRight.X);
			MaxCorner.Y = FMath::Max(MaxCorner.Y, NodeBottomRight.Y);
		}
		GraphEditor->GetGraphPanel()->JumpToRect(MinCorner, MaxCorner);
	}

	// Apply opacity
	ApplyFocusOpacity();
}

void SRigMapperDefinitionGraphEditor::ClearFocusMode()
{
	if (!bFocusMode) return;

	bFocusMode = false;
	FocusedNodes.Reset();

	// Restore all nodes to full opacity
	SGraphPanel* Panel = GraphEditor->GetGraphPanel();
	for (UEdGraphNode* Node : GraphObj->Nodes)
	{
		TSharedPtr<SGraphNode> Widget = Panel->GetNodeWidgetFromGuid(Node->NodeGuid);
		if (Widget.IsValid())
		{
			Widget->SetRenderOpacity(1.0f);
		}
	}
}

void SRigMapperDefinitionGraphEditor::ApplyFocusOpacity()
{
	SGraphPanel* Panel = GraphEditor->GetGraphPanel();
	for (UEdGraphNode* Node : GraphObj->Nodes)
	{
		if (URigMapperDefinitionEditorGraphNode* RigMapperNode = Cast<URigMapperDefinitionEditorGraphNode>(Node))
		{
			TSharedPtr<SGraphNode> Widget = Panel->GetNodeWidgetFromGuid(RigMapperNode->NodeGuid);
			if (Widget.IsValid())
			{
				Widget->SetRenderOpacity(FocusedNodes.Contains(RigMapperNode) ? 1.0f : 0.15f);
			}
		}
	}
}

void SRigMapperDefinitionGraphEditor::OnToggleFocusMode()
{
	if (bFocusMode)
	{
		ClearFocusMode();
		return;
	}

	// Get currently selected node
	const FGraphPanelSelectionSet& Selection = GraphEditor->GetSelectedNodes();
	for (UObject* Obj : Selection)
	{
		if (URigMapperDefinitionEditorGraphNode* Node =
			Cast<URigMapperDefinitionEditorGraphNode>(Obj))
		{
			SetFocusOnNode(Node);
			return;
		}
	}
}

void SRigMapperDefinitionGraphEditor::OnCreateComment()
{
	UEdGraph* CurrentGraph = GraphEditor->GetCurrentGraph();
	if (!CurrentGraph)
	{
		return;
	}

	const FGraphPanelSelectionSet& Selection = GraphEditor->GetSelectedNodes();
	TArray<UEdGraphNode*> SelectedNodes;
	SelectedNodes.Reserve(Selection.Num());
	for (UObject* Obj : Selection)
	{
		if (UEdGraphNode* Node = Cast<UEdGraphNode>(Obj))
		{
			SelectedNodes.Add(Node);
		}
	}

	const FVector2D PasteLocation = GraphEditor->GetPasteLocation2f();
	URigMapperDefinitionEditorGraphSchema::CreateCommentNode(
		CurrentGraph,
		FVector2f(static_cast<float>(PasteLocation.X), static_cast<float>(PasteLocation.Y)),
		SelectedNodes,
		/*bSelectNewNode=*/true);
}

#undef LOCTEXT_NAMESPACE

END_SLATE_FUNCTION_BUILD_OPTIMIZATION
