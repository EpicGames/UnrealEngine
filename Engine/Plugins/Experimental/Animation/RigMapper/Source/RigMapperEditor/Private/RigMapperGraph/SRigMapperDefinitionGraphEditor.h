// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GraphEditor.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API RIGMAPPEREDITOR_API

class URigMapperDefinitionEditorGraphNode;
class URigMapperDefinition;
class URigMapperDefinitionEditorGraph;
class SRigMapperValidationBanner;

/**
 * 
 */
class SRigMapperDefinitionGraphEditor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRigMapperDefinitionGraphEditor)
		{
		}

	SLATE_END_ARGS()

	UE_API virtual ~SRigMapperDefinitionGraphEditor() override;

	/** Constructs this widget with InArgs */
	UE_API void Construct(const FArguments& InArgs, URigMapperDefinition* InDefinition);

	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	UE_API void SelectNodes(const TArray<FString>& Inputs, const TArray<FString>& Features, const TArray<FString>& Outputs, const TArray<FString>& NullOutputs);
	UE_API void RebuildGraph();
	UE_API void RefreshGraphNode(URigMapperDefinitionEditorGraphNode* InNode);
	UE_API void CaptureKeyboard();

	TSharedPtr<SRigMapperValidationBanner> GetValidationBanner() const 
	{ 
		return ValidationBanner; 
	}

	TObjectPtr<URigMapperDefinitionEditorGraph> GetEditorGraphObject() const
	{
		return GraphObj;
	}

private:
	UE_API void ZoomToFitNodes(const TArray<URigMapperDefinitionEditorGraphNode*>& SelectedNodes) const;
	static UE_API void GetAllLinkedNodes(const URigMapperDefinitionEditorGraphNode* BaseNode, TArray<URigMapperDefinitionEditorGraphNode*>& LinkedNodes, bool bDescend);
	UE_API void HandleSelectionChanged(const TSet<UObject*>& Nodes);
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	void BindCommands();
	void OnDeleteSelectedNodes();
	bool CanDeleteNodes();

	void SetFocusOnNode(URigMapperDefinitionEditorGraphNode* InNode);
	void ClearFocusMode();
	void ApplyFocusOpacity();
	void OnToggleFocusMode();
	void OnCreateComment();

public:
	SGraphEditor::FOnSelectionChanged OnSelectionChanged;	
	
private:
	TSharedPtr<SGraphEditor> GraphEditor;
	TObjectPtr<URigMapperDefinitionEditorGraph> GraphObj;
	/** Commands that are bound to delegates*/
	TSharedPtr<FUICommandList> UICommandList;
	TSharedPtr<SRigMapperValidationBanner> ValidationBanner;

	/** Designates if focus mode is on*/
	bool bFocusMode = false;
	/** Nodes connected in the single chain focused by the user*/
	TSet<URigMapperDefinitionEditorGraphNode*> FocusedNodes;

	bool bSelectingNodes = false;

	bool bFocusLinkedNodes = true;

	bool bDeferZoom = false;
};

#undef UE_API
