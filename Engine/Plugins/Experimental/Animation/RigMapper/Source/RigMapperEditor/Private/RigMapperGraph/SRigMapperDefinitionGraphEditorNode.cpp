// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRigMapperDefinitionGraphEditorNode.h"

#include "RigMapperDefinitionEditorGraphNode.h"
#include "SlateOptMacros.h"
#include "Components/HorizontalBox.h"
#include "SGraphPin.h"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SRigMapperDefinitionGraphEditorNode::Construct(const FArguments& InArgs, URigMapperDefinitionEditorGraphNode* InNode)
{
	GraphNode = InNode;
	SetCursor(EMouseCursor::CardinalCross);	
	UpdateGraphNode();
}

void SRigMapperDefinitionGraphEditorNode::CreatePinWidgets()
{
	for (UEdGraphPin* Pin : GraphNode->Pins)
	{
		TSharedPtr<SGraphPin> PinWidget = CreatePinWidget(Pin);
		AddPin(PinWidget.ToSharedRef());	
	}
}

FSlateColor SRigMapperDefinitionGraphEditorNode::GetNodeColor() const
{
	if (GraphNode)
	{
		return FSlateColor(GraphNode->GetNodeTitleColor());
	}
	return FSlateColor(FLinearColor::Red);
}

FText SRigMapperDefinitionGraphEditorNode::GetNodeTitle() const
{
	if (GraphNode)
	{
		return GraphNode->GetNodeTitle(ENodeTitleType::FullTitle);
	}
	return FText::FromString("Invalid");
}

FText SRigMapperDefinitionGraphEditorNode::GetNodeSubtitle() const
{
	if (const URigMapperDefinitionEditorGraphNode* Node = Cast<URigMapperDefinitionEditorGraphNode>(GraphNode))
	{
		return Node->GetSubtitle();
	}
	return FText::GetEmpty();
}


void SRigMapperDefinitionGraphEditorNode::UpdateGraphNode()
{
	InputPins.Empty();
	OutputPins.Empty();
	
	RightNodeBox.Reset();
	LeftNodeBox.Reset();

	this->GetOrAddSlot( ENodeZone::Center )
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SBorder)
			.BorderImage( FAppStyle::GetBrush("PhysicsAssetEditor.Graph.NodeBody") )
			.BorderBackgroundColor(this, &SRigMapperDefinitionGraphEditorNode::GetNodeColor)
			.Padding(0)
			[
				SNew(SHorizontalBox)
				//.Visibility(EVisibility::HitTestInvisible)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SAssignNew(LeftNodeBox, SVerticalBox)
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				.Padding(4.0f)
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Center)
					.Padding(8, 0)
					[
						SNew(STextBlock)
						.TextStyle( FAppStyle::Get(), "Graph.Node.NodeTitle" )
						.ColorAndOpacity(FSlateColor(FLinearColor::Black))
						.Text(this, &SRigMapperDefinitionGraphEditorNode::GetNodeTitle)
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Center)
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), "NormalText")
						.ColorAndOpacity(FSlateColor(FLinearColor::Black))
						.Text(this, &SRigMapperDefinitionGraphEditorNode::GetNodeSubtitle)
						.Visibility(this, &SRigMapperDefinitionGraphEditorNode::GetShowNodeSubtitle)
					]
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SAssignNew(RightNodeBox, SVerticalBox)
				]
			]
		];

	CreatePinWidgets();
}

TSharedPtr<SGraphPin> SRigMapperDefinitionGraphEditorNode::CreatePinWidget(UEdGraphPin* Pin) const
{
	// TODO: Create our own NodeFactory create pin widget method.
	return SNew(SGraphPin, Pin);
}

void SRigMapperDefinitionGraphEditorNode::AddPin(const TSharedRef<SGraphPin>& PinToAdd)
{
	PinToAdd->SetOwner(SharedThis(this));
	PinToAdd->SetIsEditable(true);

	const UEdGraphPin* PinObj = PinToAdd->GetPinObj();
	const bool bAdvancedParameter = PinObj && PinObj->bAdvancedView;
	if (bAdvancedParameter)
	{
		PinToAdd->SetVisibility(TAttribute<EVisibility>(PinToAdd, &SGraphPin::IsPinVisibleAsAdvanced));
	}

	if (PinToAdd->GetDirection() == EEdGraphPinDirection::EGPD_Input)
	{
		LeftNodeBox->AddSlot()
			.AutoHeight()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.FillHeight(1.0f)
			[
				SNew(SOverlay)
				.Visibility(EVisibility::SelfHitTestInvisible)
				+ SOverlay::Slot()
				[
					PinToAdd
				]
			];
		InputPins.Add(PinToAdd);
	}
	else // Direction == EEdGraphPinDirection::EGPD_Output
	{
		RightNodeBox->AddSlot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.FillHeight(1.0f)
			[
				SNew(SOverlay)
				.Visibility(EVisibility::SelfHitTestInvisible)
				+ SOverlay::Slot()
				[
					PinToAdd
				]
			];
		OutputPins.Add(PinToAdd);
	}
}

const FSlateBrush* SRigMapperDefinitionGraphEditorNode::GetShadowBrush(bool bSelected) const
{
	return bSelected ? FAppStyle::GetBrush(TEXT("PhysicsAssetEditor.Graph.Node.ShadowSelected")) : FAppStyle::GetBrush(TEXT("PhysicsAssetEditor.Graph.Node.Shadow"));
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION
