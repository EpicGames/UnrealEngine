// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SGraphNode.h"

#define UE_API RIGMAPPEREDITOR_API

class URigMapperDefinitionEditorGraphNode;

/**
 * 
 */
class SRigMapperDefinitionGraphEditorNode : public SGraphNode
{
public:
	SLATE_BEGIN_ARGS(SRigMapperDefinitionGraphEditorNode)
		{
		}

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	UE_API void Construct(const FArguments& InArgs, URigMapperDefinitionEditorGraphNode* InNode);
	UE_API void CreatePinWidgets() override;

protected:
	UE_API FSlateColor GetNodeColor() const;
	UE_API FText GetNodeTitle() const;
	UE_API FText GetNodeSubtitle() const;
	EVisibility GetShowNodeSubtitle() const { return GetNodeSubtitle().IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible; };
	
	// SGraphNode interface
	UE_API virtual void UpdateGraphNode() override;
	UE_API virtual const FSlateBrush* GetShadowBrush(bool bSelected) const override;
	UE_API TSharedPtr<SGraphPin> CreatePinWidget(UEdGraphPin* Pin) const override;
	UE_API void AddPin(const TSharedRef<SGraphPin>& PinToAdd) override;
};

#undef UE_API
