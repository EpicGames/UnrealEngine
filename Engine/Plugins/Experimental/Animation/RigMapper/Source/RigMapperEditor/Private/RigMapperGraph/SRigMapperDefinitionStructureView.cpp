// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRigMapperDefinitionStructureView.h"

#include "RigMapperDefinition.h"
#include "RigMapperDefinitionEditorGraph.h"
#include "SlateOptMacros.h"
#include "Widgets/Input/SSearchBox.h"

#define LOCTEXT_NAMESPACE "RigMapperDefinitionStructureView"

const int32 SRigMapperDefinitionStructureView::NumNodeTypes = 4;
const int32 SRigMapperDefinitionStructureView::NumFeatureTypes = 3;
const FString SRigMapperDefinitionStructureView::InputsNodeName = LOCTEXT("RigMapperDefinitionStructureViewInputs","Inputs").ToString();
const FString SRigMapperDefinitionStructureView::FeaturesNodeName = LOCTEXT("RigMapperDefinitionStructureViewFeatures", "Features").ToString();
const FString SRigMapperDefinitionStructureView::MultiplyNodeName = LOCTEXT("RigMapperDefinitionStructureViewMultiplyFeatures", "Multiply").ToString();
const FString SRigMapperDefinitionStructureView::MathNodeName = LOCTEXT("RigMapperDefinitionStructureViewMathFeatures", "Math Ops").ToString();
const FString SRigMapperDefinitionStructureView::WsNodeName = LOCTEXT("RigMapperDefinitionStructureViewWeightedSumsFeatures", "Weighted Sums").ToString();
const FString SRigMapperDefinitionStructureView::SdkNodeName = LOCTEXT("RigMapperDefinitionStructureViewSDKsFeatures", "SDKs").ToString();
const FString SRigMapperDefinitionStructureView::OutputNodeName = LOCTEXT("RigMapperDefinitionStructureViewOutputs", "Outputs").ToString();
const FString SRigMapperDefinitionStructureView::NullOutputNodeName = LOCTEXT("RigMapperDefinitionStructureViewNullOutputs", "Null Outputs").ToString();


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SRigMapperDefinitionStructureView::Construct(const FArguments& InArgs, URigMapperDefinitionEditorGraph* InGraph)
{
	Graph = InGraph;
	GenerateParentNodes();
	GenerateChildrenNodes();

	TreeView = SNew(STreeView<TSharedPtr<FString>>)
		.SelectionMode(ESelectionMode::Multi)
		.HighlightParentNodesForSelection(true)
		.ReturnFocusToSelection(false)
		.OnGenerateRow(this, &SRigMapperDefinitionStructureView::OnGenerateTreeRow)
		.OnGetChildren(this, &SRigMapperDefinitionStructureView::OnGetTreeNodeChildren)
		.OnSelectionChanged(this, &SRigMapperDefinitionStructureView::HandleTreeNodesSelectionChanged)
		.TreeItemsSource(&FilteredRootNodes);

	SearchBoxFilter = MakeShared<TTextFilter<TSharedPtr<FString>>>(TTextFilter<TSharedPtr<FString>>::FItemToStringArray::CreateSP(this, &SRigMapperDefinitionStructureView::TransformElementToString));
	
	for (const TPair<TSharedPtr<FString>, TArray<TSharedPtr<FString>>>& NodeAndChildren : ParentsAndChildrenNodes)
	{
		TreeView->SetItemExpansion(NodeAndChildren.Key, true);	
	}
	
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(7.0f, 6.0f)
		.AutoHeight()
		[
			SAssignNew(SearchBox, SSearchBox)
			.OnTextChanged(this, &SRigMapperDefinitionStructureView::OnFilterTextChanged)
		]

		+ SVerticalBox::Slot()
		[
			TreeView.ToSharedRef()
		]
	]; 
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION


TSharedPtr<FString> SRigMapperDefinitionStructureView::GetParentAndChildrenNodes(ERigMapperFeatureType NodeType, TArray<TSharedPtr<FString>>& OutChildren)
{
	if (const TSharedPtr<FString>* ParentNode = ParentNodesMapping.Find(NodeType))
	{
		if (const TArray<TSharedPtr<FString>>* ChildrenNodes = ParentsAndChildrenNodes.Find(*ParentNode))
		{
			OutChildren = *ChildrenNodes;
			return *ParentNode;
		}
	}
	return nullptr;
}

TArray<TSharedPtr<FString>>* SRigMapperDefinitionStructureView::GetChildrenNodes(ERigMapperFeatureType NodeType)
{
	if (const TSharedPtr<FString>* ParentNode = ParentNodesMapping.Find(NodeType))
	{
		return ParentsAndChildrenNodes.Find(*ParentNode);
	}
	return nullptr;
}

const TArray<TSharedPtr<FString>>* SRigMapperDefinitionStructureView::GetChildrenNodes(ERigMapperFeatureType NodeType) const
{
	if (const TSharedPtr<FString>* ParentNode = ParentNodesMapping.Find(NodeType))
	{
		return ParentsAndChildrenNodes.Find(*ParentNode);
	}
	return nullptr;
}

bool SRigMapperDefinitionStructureView::SelectNode(const FString& NodeName, ERigMapperFeatureType NodeType, bool bSelected)
{
	if (const TArray<TSharedPtr<FString>>* ChildrenNodes = GetChildrenNodes(NodeType))
	{
		if (const TSharedPtr<FString>* TreeNode = ChildrenNodes->FindByPredicate(
			[&NodeName](const TSharedPtr<FString>& Item) { return Item && *Item.Get() == NodeName; }))
		{
			TreeView->SetItemSelection(*TreeNode, bSelected);
			TreeView->RequestNavigateToItem(*TreeNode);
			return true;
		}
	}
	return false;
}

void SRigMapperDefinitionStructureView::RebuildTree()
{
	TreeView->ClearSelection();
	GenerateChildrenNodes();
	
	FilteredRootNodes.Reset();
	FilterNodes(RootNodes, FilteredRootNodes);

	TreeView->RequestTreeRefresh();
}

void SRigMapperDefinitionStructureView::ClearSelection() const
{
	TreeView->ClearSelection();
}

bool SRigMapperDefinitionStructureView::IsNodeOrChildSelected(ERigMapperFeatureType NodeType, int32 ArrayIndex) const
{
	if (NodeType == ERigMapperFeatureType::Invalid)
	{
		return	IsNodeOrChildSelected(ERigMapperFeatureType::Multiply, ArrayIndex) ||
				IsNodeOrChildSelected(ERigMapperFeatureType::MathOp, ArrayIndex) ||
				IsNodeOrChildSelected(ERigMapperFeatureType::WeightedSum, ArrayIndex) ||
				IsNodeOrChildSelected(ERigMapperFeatureType::SDK, ArrayIndex);
	}
	
	const TArray<TSharedPtr<FString>>& Selection = TreeView->GetSelectedItems();
	
	if (const TArray<TSharedPtr<FString>>* ChildrenNodes = GetChildrenNodes(NodeType))
	{
		if (ArrayIndex == INDEX_NONE)
		{
			for (const TSharedPtr<FString>& Node : Selection)
			{
				if (ChildrenNodes->Contains(Node))
				{
					return true;
				}
			}
		}
		return ChildrenNodes->IsValidIndex(ArrayIndex) && Selection.Contains((*ChildrenNodes)[ArrayIndex]);	
	}
	return false;
}

bool SRigMapperDefinitionStructureView::IsSelectionEmpty() const
{
	return TreeView->GetNumItemsSelected() == 0;
}

void SRigMapperDefinitionStructureView::GenerateParentNodes()
{
	RootNodes.Reset(NumFeatureTypes);
	ParentNodesMapping.Empty(NumFeatureTypes + NumNodeTypes);
	ParentsAndChildrenNodes.Empty(NumFeatureTypes + NumNodeTypes);

	ParentsAndChildrenNodes.Add(ParentNodesMapping.Add(
		ERigMapperFeatureType::Input, RootNodes.Add_GetRef(MakeShared<FString>(InputsNodeName))));

	TArray<TSharedPtr<FString>>& FeatureEntries = ParentsAndChildrenNodes.Add(ParentNodesMapping.Add(
		ERigMapperFeatureType::Invalid, RootNodes.Add_GetRef(MakeShared<FString>(FeaturesNodeName))));
	FeatureEntries.Reserve(NumNodeTypes);

	ParentsAndChildrenNodes.Add(ParentNodesMapping.Add(
		ERigMapperFeatureType::Multiply, FeatureEntries.Add_GetRef(MakeShared<FString>(MultiplyNodeName))));

	ParentsAndChildrenNodes.Add(ParentNodesMapping.Add(
		ERigMapperFeatureType::MathOp, FeatureEntries.Add_GetRef(MakeShared<FString>(MathNodeName))));

	ParentsAndChildrenNodes.Add(ParentNodesMapping.Add(
		ERigMapperFeatureType::WeightedSum, FeatureEntries.Add_GetRef(MakeShared<FString>(WsNodeName))));
	
	ParentsAndChildrenNodes.Add(ParentNodesMapping.Add(
		ERigMapperFeatureType::SDK, FeatureEntries.Add_GetRef(MakeShared<FString>(SdkNodeName))));

	ParentsAndChildrenNodes.Add(ParentNodesMapping.Add(
		ERigMapperFeatureType::Output, RootNodes.Add_GetRef(MakeShared<FString>(OutputNodeName))));

	ParentsAndChildrenNodes.Add(ParentNodesMapping.Add(
		ERigMapperFeatureType::NullOutput, RootNodes.Add_GetRef(MakeShared<FString>(NullOutputNodeName))));

	FilteredRootNodes = RootNodes;
}

void SRigMapperDefinitionStructureView::GenerateChildrenNodes()
{
	if (Graph.IsValid() && !ParentsAndChildrenNodes.IsEmpty() && !ParentNodesMapping.IsEmpty())
	{
		TArray<TSharedPtr<FString>>* InputEntries = GetChildrenNodes(ERigMapperFeatureType::Input);
		const TArray<FString>& Inputs = Graph->GetNodesByType(ERigMapperFeatureType::Input);
		InputEntries->Reset(Inputs.Num());
		for (const FString& Input : Inputs)
		{
			InputEntries->Add(MakeShared<FString>(Input));
		}

		TArray<TSharedPtr<FString>>* MultiplyEntries = GetChildrenNodes(ERigMapperFeatureType::Multiply);
		const TArray<FString>& MultiplyFeatures = Graph->GetNodesByType(ERigMapperFeatureType::Multiply);
		MultiplyEntries->Reset(MultiplyFeatures.Num());
		for (const FString& Feature : MultiplyFeatures)
		{
			MultiplyEntries->Add(MakeShared<FString>(Feature));
		}

		TArray<TSharedPtr<FString>>* MathEntries = GetChildrenNodes(ERigMapperFeatureType::MathOp);
		const TArray<FString>& MathFeatures = Graph->GetNodesByType(ERigMapperFeatureType::MathOp);
		MathEntries->Reset(MathFeatures.Num());
		for (const FString& Feature : MathFeatures)
		{
			MathEntries->Add(MakeShared<FString>(Feature));
		}

		TArray<TSharedPtr<FString>>* WsEntries = GetChildrenNodes(ERigMapperFeatureType::WeightedSum);
		const TArray<FString>& WsFeatures = Graph->GetNodesByType(ERigMapperFeatureType::WeightedSum);
		WsEntries->Reset(WsFeatures.Num());
		for (const FString& Feature : WsFeatures)
		{
			WsEntries->Add(MakeShared<FString>(Feature));
		}

		TArray<TSharedPtr<FString>>* SdkEntries = GetChildrenNodes(ERigMapperFeatureType::SDK);
		const TArray<FString>& SdkFeatures = Graph->GetNodesByType(ERigMapperFeatureType::SDK);
		SdkEntries->Reset(SdkFeatures.Num());
		for (const FString& Feature : SdkFeatures)
		{
			SdkEntries->Add(MakeShared<FString>(Feature));
		}

		TArray<TSharedPtr<FString>>* OutputEntries = GetChildrenNodes(ERigMapperFeatureType::Output);
		const TArray<FString>& Outputs = Graph->GetNodesByType(ERigMapperFeatureType::Output);
		OutputEntries->Reset(Outputs.Num());
		for (const FString& Feature : Outputs)
		{
			OutputEntries->Add(MakeShared<FString>(Feature));
		}

		TArray<TSharedPtr<FString>>* NullOutputEntries = GetChildrenNodes(ERigMapperFeatureType::NullOutput);
		const TArray<FString>& NullOutputs = Graph->GetNodesByType(ERigMapperFeatureType::NullOutput);
		NullOutputEntries->Reset(NullOutputs.Num());
		for (const FString& Feature : NullOutputs)
		{
			NullOutputEntries->Add(MakeShared<FString>(Feature));
		}
	}
}

void SRigMapperDefinitionStructureView::HandleTreeNodesSelectionChanged(TSharedPtr<FString> Node, ESelectInfo::Type SelectInfo)
{
	TArray<FString> SelectedInputs;
	TArray<FString> SelectedFeatures;
	TArray<FString> SelectedOutputs;
	TArray<FString> SelectedNullOutputs;

	if (SelectInfo != ESelectInfo::Direct)
	{
		const TArray<TSharedPtr<FString>>& SelectedItems = TreeView->GetSelectedItems();

		auto GetSelectedNodeNames = [&SelectedItems](const TArray<TSharedPtr<FString>>* PtrArray)
		{
			TArray<FString> SelectedNames;

			for (const TSharedPtr<FString>& Item : SelectedItems)
			{
				if (PtrArray->Contains(Item))
				{
					SelectedNames.Add(*Item.Get());
				}
			}
			return SelectedNames;
		};

		SelectedInputs = GetSelectedNodeNames(GetChildrenNodes(ERigMapperFeatureType::Input));

		SelectedFeatures = GetSelectedNodeNames(GetChildrenNodes(ERigMapperFeatureType::Multiply));
		SelectedFeatures.Append(GetSelectedNodeNames(GetChildrenNodes(ERigMapperFeatureType::MathOp)));
		SelectedFeatures.Append(GetSelectedNodeNames(GetChildrenNodes(ERigMapperFeatureType::WeightedSum)));
		SelectedFeatures.Append(GetSelectedNodeNames(GetChildrenNodes(ERigMapperFeatureType::SDK)));
	
		SelectedOutputs = GetSelectedNodeNames(GetChildrenNodes(ERigMapperFeatureType::Output));
		SelectedNullOutputs = GetSelectedNodeNames(GetChildrenNodes(ERigMapperFeatureType::NullOutput));
	}

	if (OnSelectionChanged.IsBound())
	{
		OnSelectionChanged.Execute(SelectInfo, SelectedInputs, SelectedFeatures, SelectedOutputs, SelectedNullOutputs);
	}
}


void SRigMapperDefinitionStructureView::TransformElementToString(TSharedPtr<FString> String, TArray<FString>& Strings)
{
	if (String.IsValid())
	{
		Strings = { *String.Get() };
	}
}

void SRigMapperDefinitionStructureView::OnFilterTextChanged(const FText& Text)
{
	SearchBoxFilter->SetRawFilterText(Text);

	FilteredRootNodes.Reset();
	FilterNodes(RootNodes, FilteredRootNodes);
	TreeView->RequestTreeRefresh();
}

void SRigMapperDefinitionStructureView::FilterNodes(const TArray<TSharedPtr<FString>>& ParentNodes, TArray<TSharedPtr<FString>>& FilteredNodes)
{
	for (TSharedPtr<FString> ParentNode : ParentNodes)
	{
		bool ChildPassedFilter = false;
		
		if (ParentNode.IsValid() && ParentsAndChildrenNodes.Contains(ParentNode))
		{
			const TArray<TSharedPtr<FString>>& Children = ParentsAndChildrenNodes[ParentNode];

			if (!SearchBoxFilter->GetRawFilterText().IsEmpty())
			{
				ChildPassedFilter = Children.ContainsByPredicate([this](const TSharedPtr<FString>& InItem)
				{
					return SearchBoxFilter->PassesFilter(InItem);
				});
			}

			if (ChildPassedFilter)
			{
				FilteredNodes.AddUnique(ParentNode);
				TreeView->SetItemExpansion(ParentNode, true);
			}
		}
		if (!ChildPassedFilter && SearchBoxFilter->PassesFilter(ParentNode))
		{
			FilteredNodes.Add(ParentNode);
		}
	}
}

TSharedRef<ITableRow> SRigMapperDefinitionStructureView::OnGenerateTreeRow(TSharedPtr<FString> NodeName, const TSharedRef<STableViewBase>& TableViewBase)
{
	return SNew(STableRow<TSharedPtr<FString>>, TableViewBase)
		.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SceneOutliner.TableViewRow"))
		.Padding(4)
		[
			SNew(STextBlock)
			.Text(FText::FromString(*NodeName.Get()))
		];
}

void SRigMapperDefinitionStructureView::OnGetTreeNodeChildren(TSharedPtr<FString> NodeName, TArray<TSharedPtr<FString>>& Children)
{
	if (NodeName.IsValid() && ParentsAndChildrenNodes.Contains(NodeName))
	{
		const TArray<TSharedPtr<FString>>& UnfilteredChildren = ParentsAndChildrenNodes[NodeName];

		Children.Reset(UnfilteredChildren.Num());
		FilterNodes(UnfilteredChildren, Children);
	}
}

#undef LOCTEXT_NAMESPACE