// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/TextFilter.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"

#define UE_API RIGMAPPEREDITOR_API

enum class ERigMapperFeatureType : uint8;
class SSearchBox;
class URigMapperDefinitionEditorGraph;

/**
 * 
 */
class SRigMapperDefinitionStructureView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRigMapperDefinitionStructureView)
		{
		}

	SLATE_END_ARGS()

	DECLARE_DELEGATE_FiveParams(FOnSelectionChanged, ESelectInfo::Type, TArray<FString>, TArray<FString>, TArray<FString>, TArray<FString>);
	
	/** Constructs this widget with InArgs */
	UE_API void Construct(const FArguments& InArgs, URigMapperDefinitionEditorGraph* InGraph);

	UE_API bool SelectNode(const FString& NodeName, ERigMapperFeatureType NodeType, bool bSelected);
	UE_API void RebuildTree();
	UE_API void ClearSelection() const;
	UE_API bool IsNodeOrChildSelected(ERigMapperFeatureType NodeType, int32 ArrayIndex) const;
	UE_API bool IsSelectionEmpty() const;
	
public:
	FOnSelectionChanged OnSelectionChanged;

private:
	UE_API void GenerateParentNodes();
	UE_API void GenerateChildrenNodes();
	UE_API void HandleTreeNodesSelectionChanged(TSharedPtr<FString> Node, ESelectInfo::Type SelectInfo);
	UE_API void TransformElementToString(TSharedPtr<FString> String, TArray<FString>& Strings);
	UE_API void OnFilterTextChanged(const FText& Text);
	UE_API void FilterNodes(const TArray<TSharedPtr<FString>>& ParentNodes, TArray<TSharedPtr<FString>>& FilteredNodes);

	UE_API TSharedRef<ITableRow> OnGenerateTreeRow(TSharedPtr<FString> NodeName, const TSharedRef<STableViewBase>& TableViewBase);
	UE_API void OnGetTreeNodeChildren(TSharedPtr<FString> NodeName, TArray<TSharedPtr<FString>>& Children);

	UE_API TSharedPtr<FString> GetParentAndChildrenNodes(ERigMapperFeatureType NodeType, TArray<TSharedPtr<FString>>& OutChildren);
	UE_API TArray<TSharedPtr<FString>>* GetChildrenNodes(ERigMapperFeatureType NodeType);
	UE_API const TArray<TSharedPtr<FString>>* GetChildrenNodes(ERigMapperFeatureType NodeType) const;
	
private:
	static UE_API const int32 NumNodeTypes;
	static UE_API const int32 NumFeatureTypes;
	
	static UE_API const FString InputsNodeName;
	static UE_API const FString FeaturesNodeName;
	static UE_API const FString MultiplyNodeName;
	static UE_API const FString MathNodeName;
	static UE_API const FString WsNodeName;
	static UE_API const FString SdkNodeName;
	static UE_API const FString OutputNodeName;
	static UE_API const FString NullOutputNodeName;

	TWeakObjectPtr<URigMapperDefinitionEditorGraph> Graph;

	TSharedPtr<STreeView<TSharedPtr<FString>>> TreeView;
	
	TSharedPtr<SSearchBox> SearchBox;
	TSharedPtr<TTextFilter<TSharedPtr<FString>>> SearchBoxFilter;

	TArray<TSharedPtr<FString>> RootNodes;
	TArray<TSharedPtr<FString>> FilteredRootNodes;
	
	TMap<TSharedPtr<FString>, TArray<TSharedPtr<FString>>> ParentsAndChildrenNodes;
	TMap<ERigMapperFeatureType, TSharedPtr<FString>> ParentNodesMapping;
};

#undef UE_API
