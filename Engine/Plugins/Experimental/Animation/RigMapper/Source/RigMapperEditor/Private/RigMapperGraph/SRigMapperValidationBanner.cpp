// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRigMapperValidationBanner.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Styling/AppStyle.h"
#include "Logging/MessageLog.h"

#define LOCTEXT_NAMESPACE "SRigMapperValidationBanner"

void SRigMapperValidationBanner::Construct(const FArguments& InArgs)
{
	ChildSlot
		[
			SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.BorderBackgroundColor(this, &SRigMapperValidationBanner::GetBannerColor)
				.Padding(FMargin(10.f, 6.f))
				.Visibility(this, &SRigMapperValidationBanner::GetBannerVisibility)
				[
					SNew(SVerticalBox)

						// Summary row
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SHorizontalBox)

								// Status icon
								+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								.Padding(0.f, 0.f, 8.f, 0.f)
								[
									SNew(SImage)
										.Image(this, &SRigMapperValidationBanner::GetStatusIcon)
								]

								// Summary text
								+ SHorizontalBox::Slot()
								.FillWidth(1.0f)
								.VAlign(VAlign_Center)
								[
									SNew(STextBlock)
										.Text(this, &SRigMapperValidationBanner::GetSummaryText)
										.TextStyle(FAppStyle::Get(), "NormalText")
										.ColorAndOpacity(FSlateColor(FLinearColor::White))
								]

								// Expand/collapse details button
								+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								.Padding(4.f, 0.f)
								[
									SNew(SButton)
										.ButtonStyle(FAppStyle::Get(), "SimpleButton")
										.OnClicked(this, &SRigMapperValidationBanner::OnToggleDetailClicked)
										.Visibility(this, &SRigMapperValidationBanner::GetShowLogVisibility)
										.ToolTipText(LOCTEXT("ToggleDetailsTooltip", "Show/hide details"))
										[
											SNew(STextBlock)
												.Text(LOCTEXT("DetailsLabel", "Details"))
												.TextStyle(FAppStyle::Get(), "SmallText")
												.ColorAndOpacity(FSlateColor(FLinearColor::White))
										]
								]

							// Open Message Log button
							+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								.Padding(4.f, 0.f)
								[
									SNew(SButton)
										.ButtonStyle(FAppStyle::Get(), "SimpleButton")
										.OnClicked(this, &SRigMapperValidationBanner::OnShowLogClicked)
										.Visibility(this, &SRigMapperValidationBanner::GetShowLogVisibility)
										.ToolTipText(LOCTEXT("ShowLogTooltip", "Open Message Log"))
										[
											SNew(STextBlock)
												.Text(LOCTEXT("ShowLogLabel", "Show Log"))
												.TextStyle(FAppStyle::Get(), "SmallText")
												.ColorAndOpacity(FSlateColor(FLinearColor::White))
										]
								]

							// Dismiss button
							+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								.Padding(4.f, 0.f, 0.f, 0.f)
								[
									SNew(SButton)
										.ButtonStyle(FAppStyle::Get(), "SimpleButton")
										.OnClicked(this, &SRigMapperValidationBanner::OnDismissClicked)
										.ToolTipText(LOCTEXT("DismissTooltip", "Dismiss"))
										[
											SNew(STextBlock)
												.Text(FText::FromString(TEXT("\u00D7")))
												.TextStyle(FAppStyle::Get(), "NormalText")
												.ColorAndOpacity(FSlateColor(FLinearColor::White))
										]
								]
						]

					// Detail list (expandable)
					+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.f, 4.f, 0.f, 0.f)
						[
							SNew(SBox)
								.MaxDesiredHeight(150.f)
								.Visibility(this, &SRigMapperValidationBanner::GetDetailVisibility)
								[
									SNew(SBorder)
										.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
										.Padding(FMargin(6.f, 4.f))
										[
											SNew(SScrollBox)
												+ SScrollBox::Slot()
												[
													SNew(STextBlock)
														.Text(this, &SRigMapperValidationBanner::GetDetailText)
														.TextStyle(FAppStyle::Get(), "SmallText")
														.ColorAndOpacity(FSlateColor(FLinearColor::White * 0.85f))
														.AutoWrapText(true)
												]
										]
								]
						]
				]
		];
}

void SRigMapperValidationBanner::SetValidationResult(const FRigMapperValidationContext& InContext)
{
	CachedContext = InContext;
	bDetailExpanded = false;

	if (InContext.HasErrors())
	{
		State = ERigMapperValidationBannerState::Error;
	}
	else if (InContext.HasWarnings())
	{
		State = ERigMapperValidationBannerState::Warning;
	}
	else
	{
		State = ERigMapperValidationBannerState::Success;
	}
}

void SRigMapperValidationBanner::Clear()
{
	State = ERigMapperValidationBannerState::Hidden;
	CachedContext = FRigMapperValidationContext();
	bDetailExpanded = false;
}

EVisibility SRigMapperValidationBanner::GetBannerVisibility() const
{
	return State == ERigMapperValidationBannerState::Hidden ? EVisibility::Collapsed : EVisibility::Visible;
}

FSlateColor SRigMapperValidationBanner::GetBannerColor() const
{
	switch (State)
	{
	case ERigMapperValidationBannerState::Success:
		return FSlateColor(FLinearColor(0.05f, 0.35f, 0.1f, 1.0f));
	case ERigMapperValidationBannerState::Warning:
		return FSlateColor(FLinearColor(0.55f, 0.42f, 0.0f, 1.0f));
	case ERigMapperValidationBannerState::Error:
		return FSlateColor(FLinearColor(0.55f, 0.1f, 0.1f, 1.0f));
	default:
		return FSlateColor(FLinearColor::Transparent);
	}
}

const FSlateBrush* SRigMapperValidationBanner::GetStatusIcon() const
{
	switch (State)
	{
	case ERigMapperValidationBannerState::Success:
		return FAppStyle::GetBrush("Icons.SuccessWithColor");
	case ERigMapperValidationBannerState::Warning:
		return FAppStyle::GetBrush("Icons.WarningWithColor");
	case ERigMapperValidationBannerState::Error:
		return FAppStyle::GetBrush("Icons.ErrorWithColor");
	default:
		return nullptr;
	}
}

FText SRigMapperValidationBanner::GetSummaryText() const
{
	const int32 ErrorCount = CachedContext.GetErrorCount();
	const int32 WarningCount = CachedContext.GetWarningCount();

	switch (State)
	{
	case ERigMapperValidationBannerState::Success:
		return LOCTEXT("ValidationSuccess", "Definition is valid");

	case ERigMapperValidationBannerState::Warning:
		return FText::Format(
			LOCTEXT("ValidationWarning", "Definition is valid with {0} warning(s)"),
			FText::AsNumber(WarningCount));

	case ERigMapperValidationBannerState::Error:
		if (WarningCount > 0)
		{
			return FText::Format(
				LOCTEXT("ValidationErrorWithWarnings", "Validation failed \u2014 {0} error(s), {1} warning(s)"),
				FText::AsNumber(ErrorCount),
				FText::AsNumber(WarningCount));
		}
		return FText::Format(
			LOCTEXT("ValidationError", "Validation failed \u2014 {0} error(s)"),
			FText::AsNumber(ErrorCount));

	default:
		return FText::GetEmpty();
	}
}

FText SRigMapperValidationBanner::GetDetailText() const
{
	if (CachedContext.Messages.IsEmpty())
	{
		return FText::GetEmpty();
	}

	FString Combined;
	for (int32 Index = 0; Index < CachedContext.Messages.Num(); ++Index)
	{
		const FRigMapperValidationMessage& Msg = CachedContext.Messages[Index];
		const TCHAR* Prefix = (Msg.Severity == ERigMapperValidationSeverity::Error) ? TEXT("[Error]") : TEXT("[Warning]");

		if (Index > 0)
		{
			Combined += TEXT("\n");
		}
		Combined += FString::Printf(TEXT("%s %s"), Prefix, *Msg.Message);
	}
	return FText::FromString(Combined);
}

EVisibility SRigMapperValidationBanner::GetDetailVisibility() const
{
	return bDetailExpanded && CachedContext.Messages.Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SRigMapperValidationBanner::GetShowLogVisibility() const
{
	return CachedContext.Messages.Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply SRigMapperValidationBanner::OnDismissClicked()
{
	Clear();
	return FReply::Handled();
}

FReply SRigMapperValidationBanner::OnShowLogClicked()
{
	FMessageLog MessageLog("RigMapperEditor");
	for (const FRigMapperValidationMessage& Msg : CachedContext.Messages)
	{
		if (Msg.Severity == ERigMapperValidationSeverity::Error)
		{
			MessageLog.Error(FText::FromString(Msg.Message));
		}
		else
		{
			MessageLog.Warning(FText::FromString(Msg.Message));
		}
	}
	MessageLog.Open();
	return FReply::Handled();
}

FReply SRigMapperValidationBanner::OnToggleDetailClicked()
{
	bDetailExpanded = !bDetailExpanded;
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE