// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "RigMapperDefinition.h"

enum class ERigMapperValidationBannerState : uint8
{
	Hidden,
	Success,
	Warning,
	Error
};

class SRigMapperValidationBanner : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRigMapperValidationBanner) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void SetValidationResult(const FRigMapperValidationContext& InContext);
	void Clear();

private:
	EVisibility GetBannerVisibility() const;
	FSlateColor GetBannerColor() const;
	const FSlateBrush* GetStatusIcon() const;
	FText GetSummaryText() const;
	FText GetDetailText() const;
	EVisibility GetDetailVisibility() const;
	EVisibility GetShowLogVisibility() const;

	FReply OnDismissClicked();
	FReply OnShowLogClicked();
	FReply OnToggleDetailClicked();

	ERigMapperValidationBannerState State = ERigMapperValidationBannerState::Hidden;
	FRigMapperValidationContext CachedContext;
	bool bDetailExpanded = false;
};