// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigMapperLinkedDefinitionsAssetTypeActions.h"

#include "RigMapperDefinition.h"

#define LOCTEXT_NAMESPACE "RigMapperLinkedDefinitionsAssetTypeActions"

UClass* FRigMapperLinkedDefinitionsAssetTypeActions::GetSupportedClass() const
{
	return URigMapperLinkedDefinitions::StaticClass();
}

FText FRigMapperLinkedDefinitionsAssetTypeActions::GetName() const
{
	return LOCTEXT("FRigMapperDefinitionAssetTypeActionsName", "Rig Mapper Linked Definitions");
}

FColor FRigMapperLinkedDefinitionsAssetTypeActions::GetTypeColor() const
{
	return FColor::Yellow;
}

uint32 FRigMapperLinkedDefinitionsAssetTypeActions::GetCategories()
{
	return EAssetTypeCategories::Animation;
}

#undef LOCTEXT_NAMESPACE