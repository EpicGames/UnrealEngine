// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
 
#include "CoreMinimal.h"
#include "AssetTypeActions_Base.h"

#define UE_API RIGMAPPEREDITOR_API

/**
 * The asset actions for the URigMapperLinkedDefinitions data asset class and link to its asset editor toolkit
 */
class FRigMapperLinkedDefinitionsAssetTypeActions : public FAssetTypeActions_Base
{
public:
	UE_API virtual UClass* GetSupportedClass() const override;
	UE_API virtual FText GetName() const override;
	UE_API virtual FColor GetTypeColor() const override;
	UE_API virtual uint32 GetCategories() override;
};

#undef UE_API
