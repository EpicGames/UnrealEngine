// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphNode_Base.h"
#include "CoreMinimal.h"
#include "AnimNode_RigMapper.h"

#include "AnimGraphNode_RigMapper.generated.h"

#define UE_API RIGMAPPEREDITOR_API

/**
 * 
 */
UCLASS(MinimalAPI)
class UAnimGraphNode_RigMapper : public UAnimGraphNode_Base
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	FAnimNode_RigMapper Node;

public:
	// UEdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FLinearColor GetNodeBodyTintColor() const override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base interface
	UE_API virtual void ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog) override;
	UE_API virtual FEditorModeID GetEditorMode() const override;
	UE_API virtual EAnimAssetHandlerType SupportsAssetClass(const UClass* AssetClass) const override;
	UE_API virtual void CopyNodeDataToPreviewNode(FAnimNode_Base* InPreviewNode) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UAnimGraphNode_Base interface
};

#undef UE_API
