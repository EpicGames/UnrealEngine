// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/FrameRate.h"
#include "EditorSubsystem.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "RigMapperDefinition.h"
#include "RigMapperProcessor.h"

#include "RigMapperEditorSubsystem.generated.h"

#define UE_API RIGMAPPEREDITOR_API

class USkeletalMesh;
class UAnimSequence;
class ULevelSequence;
class UControlRig;
class UMovieSceneControlRigParameterSection;
class USkeleton;

/**
 * URigMapperEditorSubsystem
 * Subsystem to remap animation from/to all kinds of formats using the Rig Mapper API.
 */
UCLASS(MinimalAPI)
class URigMapperEditorSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()
	
public:
	/**
	 * Remap a CSV file using the given definitions, and optionally outputting separate files for each definition's output
	 * The following header is expected at the first line of the CSV file: curve_name, frame_number, value
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting|Animation|Rig Mapper")
	static UE_API bool ConvertCsv(const FFilePath& InputFile, const FFilePath& OutputFile, const TArray<URigMapperDefinition*>& Definitions, bool bOutputIntermediateCsvFiles = false);

	/**
	 * Import a CSV file to an existing anim sequence, remapping it using the given definitions
	 * The following header is expected at the first line of the CSV file: curve_name, frame_number, value
	 * The target sequence rate will be unchanged (see SetAnimSequenceRate if required)
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting|Animation|Rig Mapper")
	static UE_API bool ConvertCsvToAnimSequence(const FFilePath& InputFile, UAnimSequence* Target, const TArray<URigMapperDefinition*>& Definitions);

	/**
	 * Import a CSV file to a new anim sequence asset, remapping it using the given definitions
	 * The following header is expected at the first line of the CSV file: curve_name, frame_number, value
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting|Animation|Rig Mapper")
	static UE_API UAnimSequence* ConvertCsvToAnimSequenceNew(const FFilePath& InputFile, USkeletalMesh* TargetMesh, const TArray<URigMapperDefinition*>& Definitions, const FFrameRate& FrameRate, const FDirectoryPath& NewAssetPath, const FName NewAssetName);

	/**
	 * Import a CSV file to the given control rig section, remapping it using the given definitions
	 * The following header is expected at the first line of the CSV file: curve_name, frame_number, value
	 * Control Rig transform/vector2d/float channels are currently supported
	 * The target sequence rate will be unchanged
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting|Animation|Rig Mapper")
	static UE_API bool ConvertCsvToControlRigSection(const FFilePath& InputFile, UMovieSceneControlRigParameterSection* Target, const TArray<URigMapperDefinition*>& Definitions);

	/**
	 * Import a CSV file to a control rig section in a new Level Sqeuence, remapping it using the given definitions
	 * The following header is expected at the first line of the CSV file: curve_name, frame_number, value
	 * Control Rig transform/vector2d/float channels are currently supported
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting|Animation|Rig Mapper")
	static UE_API UMovieSceneControlRigParameterSection* ConvertCsvToControlRigSectionNew(const FFilePath& InputFile, USkeletalMesh* TargetMesh, const TArray<URigMapperDefinition*>& Definitions, const FFrameRate& FrameRate, const TSubclassOf<UControlRig>& ControlRigClass, const FDirectoryPath& NewAssetPath, const FName NewAssetName);

	/**
	 * Remap an anim sequence to another existing one, remapping it using the given definitions
	 * The target sequence rate will be unchanged (see SetAnimSequenceRate if required)
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting|Animation|Rig Mapper")
	static UE_API bool ConvertAnimSequence(const UAnimSequence* Source, UAnimSequence* Target, const TArray<URigMapperDefinition*>& Definitions);

	/**
	 * Remap an anim sequence to a new one, remapping it using the given definitions
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting|Animation|Rig Mapper")
	static UE_API UAnimSequence* ConvertAnimSequenceNew(const UAnimSequence* Source, USkeletalMesh* TargetMesh, const TArray<URigMapperDefinition*>& Definitions, const FDirectoryPath& NewAssetPath, const FName NewAssetName);

	/**
	 * Export a CSV file from an anim sequence, remapping it using the given definitions
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting|Animation|Rig Mapper")
	static UE_API bool ConvertAnimSequenceToCsv(const UAnimSequence* Source, const FFilePath& OutputFile, const TArray<URigMapperDefinition*>& Definitions, bool bOutputIntermediateCsvFiles = false);

	/**
	 * Convert an AnimSequence to an existing Control Rig section, remapping it using the given definitions
	 * Control Rig transform/vector2d/float channels are currently supported
	 * The target sequence rate will be unchanged
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting|Animation|Rig Mapper")
	static UE_API bool ConvertAnimSequenceToControlRigSection(const UAnimSequence* Source, UMovieSceneControlRigParameterSection* Target, const TArray<URigMapperDefinition*>& Definitions);

	/**
	 * Convert an AnimSequence to a Control Rig section in a new Level Sequence, remapping it using the given definitions
	 * Control Rig transform/vector2d/float channels are currently supported
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting|Animation|Rig Mapper")
	static UE_API UMovieSceneControlRigParameterSection* ConvertAnimSequenceToControlRigSectionNew(const UAnimSequence* Source, USkeletalMesh* TargetMesh, const TArray<URigMapperDefinition*>& Definitions, const TSubclassOf<UControlRig>& ControlRigClass, const FDirectoryPath& NewAssetPath, const FName NewAssetName);

	/**
	 * Convert a Control Rig section to another existing one, remapping it using the given definitions
	 * Control Rig transform/vector2d/float channels are currently supported
	 * The target sequence rate will be unchanged
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting|Animation|Rig Mapper")
	static UE_API bool ConvertControlRigSection(const UMovieSceneControlRigParameterSection* Source, UMovieSceneControlRigParameterSection* Target, const TArray<URigMapperDefinition*>& Definitions);

	/**
	 * Convert a Control Rig section to a section in a new Level Sequence, remapping it using the given definitions
	 * Control Rig transform/vector2d/float channels are currently supported
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting|Animation|Rig Mapper")
	static UE_API UMovieSceneControlRigParameterSection* ConvertControlRigSectionNew(const UMovieSceneControlRigParameterSection* Source, USkeletalMesh* TargetMesh, const TArray<URigMapperDefinition*>& Definitions, const TSubclassOf<UControlRig>& ControlRigClass, const FDirectoryPath& NewAssetPath, const FName NewAssetName);

	/**
	 * Export a CSV file from a Control Rig Section, remapping it using the given definitions
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting|Animation|Rig Mapper")
	static UE_API bool ConvertControlRigSectionToCsv(const UMovieSceneControlRigParameterSection* Source, const FFilePath& OutputFile, const TArray<URigMapperDefinition*>& Definitions, bool bOutputIntermediateCsvFiles = false);

	/**
	 * Convert a Control Rig section to an existing AnimSequence, remapping it using the given definitions
	 * The target sequence rate will be unchanged
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting|Animation|Rig Mapper")
	static UE_API bool ConvertControlRigSectionToAnimSequence(const UMovieSceneControlRigParameterSection* Source, UAnimSequence* Target, const TArray<URigMapperDefinition*>& Definitions);

	/**
	 * Convert a Control Rig section to a new AnimSequence asset, remapping it using the given definitions
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting|Animation|Rig Mapper")
	static UE_API UAnimSequence* ConvertControlRigSectionToAnimSequenceNew(const UMovieSceneControlRigParameterSection* Source, USkeletalMesh* TargetMesh, const TArray<URigMapperDefinition*>& Definitions, const FDirectoryPath& NewAssetPath, const FName NewAssetName);
	
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting|Animation|Rig Mapper")
	static UE_API ULevelSequence* GetSequenceFromSection(const UMovieSceneControlRigParameterSection* Section);

	UFUNCTION(BlueprintCallable, Category = "Editor Scripting|Animation|Rig Mapper")
	static UE_API TArray<UMovieSceneControlRigParameterSection*> GetSectionsFromSequence(ULevelSequence* Sequence);
	
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting|Animation|Rig Mapper")
	static UE_API FFrameRate GetAnimSequenceRate(const UAnimSequence* AnimSequence);

	UFUNCTION(BlueprintCallable, Category = "Editor Scripting|Animation|Rig Mapper")
	static UE_API void SetAnimSequenceRate(UAnimSequence* AnimSequence, FFrameRate FrameRate, bool bSetImportProperties);

	using FFrameValues = TArray<FRigMapperProcessor::FPoseValues>;
	/**
	 * Load curve values from animation sequence (not blueprint callable)
	 */
	static UE_API bool LoadCurveValuesFromAnimSequence(const UAnimSequence* InSource, const USkeleton* InSourceSkeleton, TArray<FName>& OutCurveNames, TArray<FFrameTime>& OutFrameTimes, FFrameValues& OutCurveValuesPerFrame, TArray<int32> & OutCurveFlags, TArray<FLinearColor> & OutCurveColors);


private:
	using FFramePoses = TArray<FRigMapperProcessor::FPose>;
	
	static UE_API bool ConvertCurveValuesToCsv(TArray<FString>& CurveNames, const TArray<FFrameTime>& FrameTimes, FFrameValues& CurveValuesPerFrame, const FFilePath& OutputFile, const TArray<URigMapperDefinition*>& Definitions, bool bOutputIntermediateCsvFiles);

	static UE_API bool WriteCurveValuesToCsv(const FFilePath& OutputFile, const TArray<FString>& CurveNames, const TArray<FFrameTime>& FrameTimes, const FFrameValues& CurveValuesPerFrame);
	
	static UE_API bool LoadCurveValuesFromCsv(const FFilePath& InputFile, TArray<FString>& CurveNames, TArray<FFrameTime>& FrameTimes, FFrameValues& CurveValuesPerFrame);
	
	static UE_API bool ConvertCurveValuesToAnimSequence(TArray<FString>& CurveNames, const TArray<FFrameTime>& FrameTimes, FFrameValues& CurveValuesPerFrame, UAnimSequence* Target, const TArray<URigMapperDefinition*>& Definitions);

	static UE_API bool AddCurveValuesToAnimSequence(UAnimSequence* Target, const TArray<FString>& CurveNames, const TArray<FFrameTime>& FrameTimes, const FFrameValues& CurveValuesPerFrame);

	static UE_API bool LoadCurveValuesFromAnimSequence(const UAnimSequence* Source, TArray<FString>& CurveNames, TArray<FFrameTime>& FrameTimes, FFrameValues& CurveValuesPerFrame);
	
	static UE_API bool ConvertCurveValuesToControlRigSection(TArray<FString>& CurveNames, const TArray<FFrameTime>& FrameTimes, FFrameValues& CurveValuesPerFrame, UMovieSceneControlRigParameterSection* Target, const TArray<URigMapperDefinition*>& Definitions);
	
	static UE_API bool AddCurveValuesToControlRigSection(UMovieSceneControlRigParameterSection* Target, const TArray<FString>& CurveNames, const TArray<FFrameTime>& FrameTimes, const FFrameValues& CurveValuesPerFrame);

	static UE_API bool LoadCurveValuesFromControlRigSection(const UMovieSceneControlRigParameterSection* Source, TArray<FString>& CurveNames, TArray<FFrameTime>& FrameTimes, FFrameValues& CurveValuesPerFrame, const TArray<FString>& InputNames);

	static UE_API bool BakeSparseKeys(const FFramePoses& Poses, const TArray<FString>& CurveNames, TArray<FFrameTime>& FrameTimes, FFrameValues& CurveValuesPerFrame);

	static UE_API void SparseBakeCurve(const FString& CurveName, int32 CurveIndex, float CurveValue, FFrameValues& CurveValuesPerFrame, const TArray<FFrameTime>& FrameTimesInOrder, int32 FrameIndex, const FFramePoses& Poses, const TArray<FFrameTime>& FrameTimes, const int32 ActualFrameIndex);
};

#undef UE_API
