// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class RigMapperEditor : ModuleRules
{
	public RigMapperEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"EditorSubsystem",
				"UnrealEd",
				"GraphEditor",
				"Engine",
				"RigMapper"
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"AnimGraph",
				"BlueprintGraph",
				"ControlRig",
				"ControlRigDeveloper",
				"LevelSequence",
				"MovieScene",
				"MovieSceneTracks",
				"TargetPlatform",
				"AssetTools",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"ContentBrowser",
				"MessageLog",
				"InputCore",
				"Sequencer"
			});
	}
}
